{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Animation Sharing",
	"Description": "Plugin to create Shared Animation systems using the Leader-Follower pose functionality",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AnimationSharing",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "AnimationSharingEd",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "SignificanceManager",
			"Enabled": true
		}
	],
	"IsExperimentalVersion": false
}