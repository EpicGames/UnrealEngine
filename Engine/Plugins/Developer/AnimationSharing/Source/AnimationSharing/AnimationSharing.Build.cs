// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AnimationSharing : ModuleRules
{
	public AnimationSharing(ReadOnlyTargetRules Target) : base(Target)
	{
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"SignificanceManager"
            }
		);

        if (Target.Type == TargetType.Editor)
        {
            PrivateDependencyModuleNames.Add("TargetPlatform");
        }

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
