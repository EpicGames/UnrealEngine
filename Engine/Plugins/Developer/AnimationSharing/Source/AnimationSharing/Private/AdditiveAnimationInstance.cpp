// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdditiveAnimationInstance.h"
#include "AnimationSharingInstances.h"
#include "AnimationSharingManager.h"
#include "Animation/AnimSequence.h"

FAdditiveAnimationInstance::FAdditiveAnimationInstance() : SkeletalMeshComponent(nullptr), AdditiveInstance(nullptr), AdditiveAnimationSequence(nullptr), BaseComponent(nullptr), bLoopingState(false)
{
}

void FAdditiveAnimationInstance::Initialise(USkeletalMeshComponent* InSkeletalMeshComponent, UClass* InAnimationBPClass)
{
	if (InSkeletalMeshComponent)
	{
		SkeletalMeshComponent = InSkeletalMeshComponent;
		if (InAnimationBPClass)
		{
			SkeletalMeshComponent->SetAnimInstanceClass(InAnimationBPClass);
			AdditiveInstance = Cast<UAnimSharingAdditiveInstance>(SkeletalMeshComponent->GetAnimInstance());
		}
		
		SkeletalMeshComponent->SetComponentTickEnabled(false);	
		SkeletalMeshComponent->SetForcedLOD(1);
		SkeletalMeshComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	}
}

void FAdditiveAnimationInstance::Setup(USkeletalMeshComponent* InBaseComponent, UAnimSequence* InAnimSequence)
{
	UAnimationSharingManager::SetDebugMaterial(SkeletalMeshComponent, 1);
	SkeletalMeshComponent->SetComponentTickEnabled(true);
	if (AdditiveInstance)
	{
		AdditiveInstance->BaseComponent = BaseComponent = InBaseComponent;
		AdditiveInstance->AdditiveAnimation = AdditiveAnimationSequence = InAnimSequence;
		AdditiveInstance->Alpha = 1.0f;
		AdditiveInstance->bStateBool = bLoopingState = true;

		SkeletalMeshComponent->AddTickPrerequisiteComponent(BaseComponent);
	}
}

void FAdditiveAnimationInstance::UpdateBaseComponent(USkeletalMeshComponent* InBaseComponent)
{
	if (AdditiveInstance)
	{
		SkeletalMeshComponent->RemoveTickPrerequisiteComponent(BaseComponent);
		AdditiveInstance->BaseComponent = BaseComponent = InBaseComponent;
		SkeletalMeshComponent->AddTickPrerequisiteComponent(BaseComponent);
	}
}

void FAdditiveAnimationInstance::Stop()
{
	if (AdditiveInstance)
	{
		UAnimationSharingManager::SetDebugMaterial(SkeletalMeshComponent, 0);
		SkeletalMeshComponent->SetComponentTickEnabled(false);
		SkeletalMeshComponent->RemoveTickPrerequisiteComponent(BaseComponent);
	}
}

void FAdditiveAnimationInstance::Start()
{
	if (AdditiveInstance)
	{
		AdditiveInstance->bStateBool = bLoopingState = false;
	}
}

USkeletalMeshComponent* FAdditiveAnimationInstance::GetBaseComponent() const
{
	return BaseComponent;
}

USkeletalMeshComponent* FAdditiveAnimationInstance::GetComponent() const
{
	return SkeletalMeshComponent;
}

