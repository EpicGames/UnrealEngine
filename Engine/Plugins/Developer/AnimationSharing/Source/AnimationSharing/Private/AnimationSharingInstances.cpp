// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationSharingInstances.h"
#include "AnimationSharingManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationSharingInstances)

UAnimSharingStateInstance::UAnimSharingStateInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), AnimationToPlay(nullptr), PermutationTimeOffset(0.f), PlayRate(1.f), StateIndex(INDEX_NONE), ComponentIndex(INDEX_NONE), Instance(nullptr)
{
	
}

void UAnimSharingStateInstance::GetInstancedActors(TArray<class AActor*>& Actors)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GetInstancedActors);
	if (Instance && Instance->PerStateData.IsValidIndex(StateIndex))
	{
		FPerStateData& StateData = Instance->PerStateData[StateIndex];
		if (StateData.Components.IsValidIndex(ComponentIndex))
		{
			USkeletalMeshComponent* Component = StateData.Components[ComponentIndex];
			const TArray<TWeakObjectPtr<USkinnedMeshComponent>>& FollowerComponents = Component->GetFollowerPoseComponents();

			Algo::TransformIf(FollowerComponents, Actors,
				[&Actors](const TWeakObjectPtr<USkinnedMeshComponent>& WeakPtr)
			{
				// Needs to be valid and unique
				return WeakPtr.IsValid() && !Actors.Contains(WeakPtr->GetOwner());
			},
				[](const TWeakObjectPtr<USkinnedMeshComponent>& WeakPtr)
			{
				return WeakPtr->GetOwner();
			});
		}
	}
}

UAnimSharingTransitionInstance::UAnimSharingTransitionInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer),  FromComponent(nullptr), ToComponent(nullptr), BlendTime(.5f), bBlendBool(false)
{
}

UAnimSharingAdditiveInstance::UAnimSharingAdditiveInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

