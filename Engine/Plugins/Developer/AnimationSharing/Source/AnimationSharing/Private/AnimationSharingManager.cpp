// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationSharingManager.h"
#include "Animation/Skeleton.h"
#include "AnimationSharingModule.h"
#include "Engine/SkeletalMesh.h"
#include "TransitionBlendInstance.h"
#include "Animation/AnimSequence.h"
#include "SignificanceManager.h"
#include "AnimationSharingSetup.h"
#include "AnimationSharingInstances.h"

#include "Materials/MaterialInterface.h"
#include "DrawDebugHelpers.h"
#include "Engine/Engine.h"

#if UE_BUILD_DEVELOPMENT
#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/AnimNode_RelevantAssetPlayerBase.h"
#endif // #if UE_BUILD_DEVELOPMENT

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationSharingManager)

#if WITH_EDITOR
#endif // WITH_EDITOR

DEFINE_LOG_CATEGORY(LogAnimationSharing);

DECLARE_CYCLE_STAT(TEXT("Tick"), STAT_AnimationSharing_Tick, STATGROUP_AnimationSharing);
DECLARE_CYCLE_STAT(TEXT("UpdateBlends"), STAT_AnimationSharing_UpdateBlends, STATGROUP_AnimationSharing);
DECLARE_CYCLE_STAT(TEXT("UpdateOnDemands"), STAT_AnimationSharing_UpdateOnDemands, STATGROUP_AnimationSharing);
DECLARE_CYCLE_STAT(TEXT("UpdateAdditives"), STAT_AnimationSharing_UpdateAdditives, STATGROUP_AnimationSharing);
DECLARE_CYCLE_STAT(TEXT("TickActorStates"), STAT_AnimationSharing_TickActorStates, STATGROUP_AnimationSharing);
DECLARE_CYCLE_STAT(TEXT("KickoffInstances"), STAT_AnimationSharing_KickoffInstances, STATGROUP_AnimationSharing);

DECLARE_DWORD_COUNTER_STAT(TEXT("NumBlends"), STAT_AnimationSharing_NumBlends, STATGROUP_AnimationSharing);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumOnDemands"), STAT_AnimationSharing_NumOnDemands, STATGROUP_AnimationSharing);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumActors"), STAT_AnimationSharing_NumActors, STATGROUP_AnimationSharing);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumComponent"), STAT_AnimationSharing_NumComponent, STATGROUP_AnimationSharing);

static int32 GAnimationSharingDebugging = 0;
static FAutoConsoleVariableRef CVarAnimSharing_DebugStates(
	TEXT("a.Sharing.DebugStates"),
	GAnimationSharingDebugging,
	TEXT("Values: 0/1/2/3\n")
	TEXT("Controls whether and which animation sharing debug features are enabled.\n")
	TEXT("0: Turned off.\n")
	TEXT("1: Turns on active leader-components and blend with material coloring, and printing state information for each actor above their capsule.\n")
	TEXT("2: Turns printing state information about currently active animation states, blend etc. Also enables line drawing from follower-components to currently assigned leader components.\n")
	TEXT("3: Enable printing of extra state info, including currently playing animations (for anim blueprints this will only be the first relevant anim player on graph 'AnimGraph')."),
	ECVF_Cheat);

static int32 GAnimationSharingDisableDebugMaterials = 0;
static FAutoConsoleVariableRef CVarAnimSharing_DisableDebugMaterials(
	TEXT("a.Sharing.DisableDebugMaterials"),
	GAnimationSharingDisableDebugMaterials,
	TEXT("Values: 0/1\n")
	TEXT("Controls whether material coloring should be disabled when anim sharing debug is enabled.\n"),
	ECVF_Cheat);

static float GAnimationSharingDebugFontScale = 1.0f;
static FAutoConsoleVariableRef CVarAnimSharing_FontScale(
	TEXT("a.Sharing.DebugFontScale"),
	GAnimationSharingDebugFontScale,
	TEXT("Scale of font used for anim sharing debug.\n"),
	ECVF_Cheat);

static int32 GAnimationSharingEnabled = 1;
static FAutoConsoleCommandWithWorldAndArgs CVarAnimSharing_Enabled(
	TEXT("a.Sharing.Enabled"),
	TEXT("Arguments: 0/1\n")
	TEXT("Controls whether the animation sharing is enabled."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		if (Args.Num() != 0)
		{
			const bool bShouldBeEnabled = Args[0].ToBool();
			if (!bShouldBeEnabled && GAnimationSharingEnabled && World)
			{
				/** Need to unregister actors here*/
				UAnimationSharingManager* Manager = FAnimSharingModule::Get(World);
				if (Manager)
				{
					Manager->ClearActorData();
				}
			}

			GAnimationSharingEnabled = bShouldBeEnabled;
			UE_LOGF(LogAnimationSharing, Log, "Animation Sharing System - %ls", GAnimationSharingEnabled ? TEXT("Enabled") : TEXT("Disabled"));
		}
	}),
	ECVF_Cheat);

#if !UE_BUILD_SHIPPING
static int32 GLeaderComponentsVisible = 0;
static FAutoConsoleCommandWithWorldAndArgs CVarAnimSharing_ToggleVisibility(
	TEXT("a.Sharing.ToggleVisibility"),
	TEXT("Toggles the visibility of the Leader Pose Components."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
		const bool bShouldBeVisible = !GLeaderComponentsVisible;

		/** Need to unregister actors here*/
		UAnimationSharingManager* Manager = FAnimSharingModule::Get(World);
		if (Manager)
		{
			Manager->SetLeaderComponentsVisibility(bShouldBeVisible);
		}

		GLeaderComponentsVisible = bShouldBeVisible;
	}),
	ECVF_Cheat);

#else
static const int32 GLeaderComponentsVisible = 0;
#endif

#if WITH_EDITOR
static TAutoConsoleVariable<FString> CVarAnimSharing_PreviewScalabilityPlatform(
	TEXT("a.Sharing.ScalabilityPlatform"),
	"",
	TEXT("Controls which platform should be used when retrieving per platform scalability settings.\n")
	TEXT("Empty: Current platform.\n")
	TEXT("Name of Platform\n")
	TEXT("Name of Platform Group\n"),
	ECVF_Cheat);
#endif

#define LOG_STATES 0
#define CSV_STATS 0
#define DETAIL_STATS 0

#if DEBUG_MATERIALS
TArray<TObjectPtr<UMaterialInterface>> UAnimationSharingManager::DebugMaterials;
#endif

void UAnimationSharingManager::BeginDestroy()
{
	Super::BeginDestroy();

	PerSkeletonData.Empty();
	AnimationSharingSetups.Empty();
	Skeletons.Empty();
	
	// Unregister tick function
	TickFunction.UnRegisterTickFunction();
	TickFunction.Manager = nullptr;
}

UWorld* UAnimationSharingManager::GetWorld() const
{
	return Cast<UWorld>(GetOuter());
}

UAnimationSharingManager* UAnimationSharingManager::GetAnimationSharingManager(UObject* WorldContextObject)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		return GetManagerForWorld(World);
	}

	return nullptr;
}

UAnimationSharingManager* UAnimationSharingManager::GetManagerForWorld(UWorld* InWorld)
{
	return FAnimSharingModule::Get(InWorld);
}

void FTickAnimationSharingFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	if (ensure(Manager))
	{
		Manager->Tick(DeltaTime);
	}
}

FString FTickAnimationSharingFunction::DiagnosticMessage()
{
	return TEXT("FTickAnimationSharingFunction");
}

FName FTickAnimationSharingFunction::DiagnosticContext(bool bDetailed)
{
	return FName(TEXT("TickAnimationSharing"));
}


FTickAnimationSharingFunction& UAnimationSharingManager::GetTickFunction()
{
	return TickFunction;
}

void UAnimationSharingManager::Initialise(const UAnimationSharingSetup* InSetup)
{
	TickFunction.Manager = this;
	TickFunction.RegisterTickFunction(GetWorld()->PersistentLevel);

	// Debug materials
#if DEBUG_MATERIALS 
	DebugMaterials.Empty();
	{
		UMaterialInterface* RedMaterial = LoadObject<UMaterialInterface>(nullptr, TEXT("/AnimationSharing/AnimSharingRed.AnimSharingRed"));
		DebugMaterials.Add(RedMaterial);
		UMaterialInterface* GreenMaterial = LoadObject<UMaterialInterface>(nullptr, TEXT("/AnimationSharing/AnimSharingGreen.AnimSharingGreen"));
		DebugMaterials.Add(GreenMaterial);
		UMaterialInterface* BlueMaterial = LoadObject<UMaterialInterface>(nullptr, TEXT("/AnimationSharing/AnimSharingBlue.AnimSharingBlue"));
		DebugMaterials.Add(BlueMaterial);
	}
#endif 

	if (InSetup)
	{
		AddSetup(InSetup);
	}
}

void UAnimationSharingManager::AddSetup(const UAnimationSharingSetup* InSetup)
{
	// check setup doesn't already exist
	if (InSetup)
	{
		if (!AnimationSharingSetups.Contains(InSetup))
		{
			AnimationSharingSetups.Add(InSetup);

			FAnimationSharingScalability NewScalabilitySettings = InSetup->ScalabilitySettings;

	#if WITH_EDITOR
			// Update local copy defaults with current platform value
			const FName PlatformName = UAnimationSharingManager::GetPlatformName();
			NewScalabilitySettings.UseBlendTransitions = NewScalabilitySettings.UseBlendTransitions.GetValueForPlatform(PlatformName);
			NewScalabilitySettings.BlendSignificanceValue = NewScalabilitySettings.BlendSignificanceValue.GetValueForPlatform(PlatformName);
			NewScalabilitySettings.MaximumNumberConcurrentBlends = NewScalabilitySettings.MaximumNumberConcurrentBlends.GetValueForPlatform(PlatformName);
			NewScalabilitySettings.TickSignificanceValue = NewScalabilitySettings.TickSignificanceValue.GetValueForPlatform(PlatformName);
			NewScalabilitySettings.RequireCurvesDuringBlends = NewScalabilitySettings.RequireCurvesDuringBlends.GetValueForPlatform(PlatformName);
	#endif
		
			for (const FPerSkeletonAnimationSharingSetup& SkeletonSetup : InSetup->SkeletonSetups)
			{
				SetupPerSkeletonData(SkeletonSetup, InSetup, NewScalabilitySettings);
			}

			// do broadcast to let interested parties know that a setup was added
			FAnimSharingModule::GetOnAnimationSharingManagerSetupAdded().Broadcast(this, GetWorld());
		}
		else
		{
			UE_LOGF(LogAnimationSharing, Error, "Passed duplicate UAnimationSharingSetup '%ls' to add to AnimationSharingManager!", *InSetup->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationSharing, Error, "Passed null UAnimationSharingSetup to add to AnimationSharingManager!");
	}
}

void UAnimationSharingManager::RemoveSetup(const UAnimationSharingSetup* InSetup)
{
	if (InSetup)
	{
		const int32 SetupIdx = AnimationSharingSetups.IndexOfByKey(InSetup);
		if (SetupIdx != INDEX_NONE)
		{
			for (int32 PerSkeletonDataIdx = 0; PerSkeletonDataIdx < PerSkeletonData.Num() ; ++PerSkeletonDataIdx)
			{
				// if skeleton instance matches setup being removed, remove it
				if (PerSkeletonData[PerSkeletonDataIdx]->OwnerSetup == InSetup)
				{
					// check no associated actors are registered
					if (PerSkeletonData[PerSkeletonDataIdx]->RegisteredActors.Num() > 0)
					{
						UE_LOGF(LogAnimationSharing, Error, "There are still actors registered to skeleton '%ls' when animation sharing setup '%ls' is being removed.", 
							(Skeletons[PerSkeletonDataIdx]) ? *Skeletons[PerSkeletonDataIdx]->GetName() : TEXT("NULL"), 
							*InSetup->GetName());
					}

					PerSkeletonData[PerSkeletonDataIdx]->UnregisterAllActors();

					PerSkeletonData.RemoveAt(PerSkeletonDataIdx);
					Skeletons.RemoveAt(PerSkeletonDataIdx);
					PerSkeletonDataIdx--;
				}
			}

			AnimationSharingSetups.Remove(InSetup);

			// recreate SkeletonIDToSkeletonIndexMap
			SkeletonIDToSkeletonIndexMap.Empty(PerSkeletonData.Num());
			for (int32 PerSkeletonDataIdx = 0; PerSkeletonDataIdx < PerSkeletonData.Num(); ++PerSkeletonDataIdx)
			{				
				SkeletonIDToSkeletonIndexMap.Add(static_cast<uint8>(PerSkeletonData[PerSkeletonDataIdx]->SkeletonID), static_cast<uint8>(PerSkeletonDataIdx));
			}
		}
		else
		{
			UE_LOGF(LogAnimationSharing, Error, "Passed UAnimationSharingSetup '%ls' to remove from AnimationSharingManager that was not registered!", *InSetup->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationSharing, Error, "Passed null UAnimationSharingSetup to remove from AnimationSharingManager!");
	}
}

const FAnimationSharingScalability& UAnimationSharingManager::GetScalabilitySettings(const UAnimationSharingSetup* InSetup /*= nullptr*/) const
{
	if (InSetup != nullptr)
	{
		const TObjectPtr<UAnimSharingInstance>* FirstMatchingSharingInstance = PerSkeletonData.FindByPredicate([InSetup](TObjectPtr<UAnimSharingInstance>& Instance){
			return (Instance != nullptr && Instance->OwnerSetup == InSetup);
		});
		
		if (FirstMatchingSharingInstance != nullptr && *FirstMatchingSharingInstance != nullptr)
		{
			return (*FirstMatchingSharingInstance)->ScalabilitySettings;
		}
	}
	else if (!PerSkeletonData.IsEmpty())
	{
		return PerSkeletonData[0]->ScalabilitySettings;
	}

	// else drop down to default settings.
	return ScalabilitySettings;
}

UAnimSharingInstance* UAnimationSharingManager::CreateAnimSharingInstance()
{
	return NewObject<UAnimSharingInstance>(this);
}

// deprecated version
void UAnimationSharingManager::SetupPerSkeletonData(const FPerSkeletonAnimationSharingSetup& SkeletonSetup)
{
	SetupPerSkeletonData(SkeletonSetup, nullptr, FAnimationSharingScalability());
}

void UAnimationSharingManager::SetupPerSkeletonData(const FPerSkeletonAnimationSharingSetup& SkeletonSetup, const UAnimationSharingSetup* SharingSetup, const FAnimationSharingScalability& ScalabilitySettingsForSkeleton)
{
	const USkeleton* Skeleton = SkeletonSetup.Skeleton;
	if (PerSkeletonData.Num() < MaxSkeletonIndex)
	{
		UAnimationSharingStateProcessor* Processor = SkeletonSetup.StateProcessorClass ? SkeletonSetup.StateProcessorClass->GetDefaultObject<UAnimationSharingStateProcessor>() : nullptr;
		UEnum* StateEnum = Processor ? Processor->GetAnimationStateEnum() : nullptr;
		if (Skeleton && StateEnum && Processor)
		{
			UAnimSharingInstance* Data = CreateAnimSharingInstance();
		
			if (!Skeletons.Contains(Skeleton))
			{
				// Try and setup up instance using provided setup data
				const uint32 SkeletonID = NextSkeletonID++;
				if (Data->Setup(this, SkeletonSetup, &ScalabilitySettingsForSkeleton, SkeletonID))
				{
					Data->OwnerSetup = SharingSetup;
					SkeletonIDToSkeletonIndexMap.Add(static_cast<uint8>(SkeletonID), static_cast<uint8>(PerSkeletonData.Num()));
				
					PerSkeletonData.Add(Data);
					Skeletons.Add(Skeleton);
				}
				else
				{
					UE_LOGF(LogAnimationSharing, Error, "Errors found when initializing Animation Sharing Data for Skeleton (%ls)!",
						Skeleton ? *Skeleton->GetName() : TEXT("None"));
				}

				if (NextSkeletonID > MaxSkeletonID)
				{
					UE_LOGF(LogAnimationSharing, Error, "Reached max number of skeletons IDs supported by anim sharing manager. Skeleton Id will be rolled over!");
					NextSkeletonID = 0;
				}
			}
			else
			{
				UE_LOGF(LogAnimationSharing, Error, "Skeleton (%ls) is being registered by sharing setup (%ls) but skeleton was already added previously!",
					Skeleton ? *Skeleton->GetName() : TEXT("None"), SharingSetup ? *SharingSetup->GetName() : TEXT("None"));
			}
		}
		else
		{
			UE_LOGF(LogAnimationSharing, Error, "Invalid Skeleton (%ls), State Enum (%ls) or State Processor (%ls)!", 
				Skeleton ? *Skeleton->GetName() : TEXT("None"),
				StateEnum ? *StateEnum->GetName() : TEXT("None"),
				Processor ? *Processor->GetName() : TEXT("None"));
		}
	}
	else
	{
		UE_LOGF(LogAnimationSharing, Error, "Skeleton (%ls) could not be added as reached max number of skeletons supported by anim sharing manager!",
			Skeleton ? *Skeleton->GetName() : TEXT("None"));
	}
}

uint32 UAnimationSharingManager::CreateActorHandle(uint8 SkeletonID, uint32 ActorIndex) const
{
	ensureMsgf(ActorIndex <= 0xFFFFFF, TEXT("Invalid Actor Handle due to overflow"));
	return (SkeletonID << 24) | ActorIndex;
}

uint8 UAnimationSharingManager::GetSkeletonIndexFromHandle(uint32 InHandle) const
{
	const uint8 SkeletonID = (InHandle & 0xFF000000) >> 24;
	const uint8* SkeletonIndex = SkeletonIDToSkeletonIndexMap.Find(SkeletonID);

	if (SkeletonIndex)
	{
		return *SkeletonIndex;
	}

	return static_cast<uint8>(MaxSkeletonIndex);
}

uint32 UAnimationSharingManager::GetActorIndexFromHandle(uint32 InHandle) const
{
	return (InHandle & 0x00FFFFFF);
}

void UAnimationSharingManager::Tick(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimationSharing_Tick);
	
	const float WorldTime = static_cast<float>(GetWorld()->GetTimeSeconds());

	/** Keeping track of currently running instances / animations for debugging purposes */
	int32 TotalNumBlends = 0;
	int32 TotalNumOnDemands = 0;
	int32 TotalNumComponents = 0;
	int32 TotalNumActors = 0;

	int32 TotalNumRunningStates = 0;
	int32 TotalNumRunningComponents = 0;

	/** Iterator over all Skeleton setups */
	for (int32 Index = 0; Index < PerSkeletonData.Num(); ++Index)
	{
		UAnimSharingInstance* Instance = PerSkeletonData[Index];
		Instance->WorldTime = WorldTime;

		/** Tick both Blend and On-Demand instances first, as they could be finishing */
		Instance->TickBlendInstances();
		Instance->TickOnDemandInstances();
		Instance->TickAdditiveInstances();

		/** Tick actor states */
		Instance->TickActorStates();

		/** Setup and start any blending transitions created while ticking the actor states */
		Instance->KickoffInstances();

#if !UE_BUILD_SHIPPING
		if (GAnimationSharingDebugging >= 1)
		{
			Instance->TickDebugInformation();
		}
#endif
		/** Tick the animation states to determine which components should be turned on/off */
		Instance->TickAnimationStates();

#if DETAIL_STATS
		/** Stat counters */
		TotalNumOnDemands += Instance->OnDemandInstances.Num();
		TotalNumBlends += Instance->BlendInstances.Num();
		TotalNumActors += Instance->PerActorData.Num();
		TotalNumComponents += Instance->PerComponentData.Num();

		for (FPerStateData& StateData : Instance->PerStateData)
		{
			if (StateData.InUseComponentFrameBits.Contains(true))
			{
				++TotalNumRunningStates;
			}

			for (int32 ComponentIndex = 0; ComponentIndex < StateData.PreviousInUseComponentFrameBits.Num(); ++ComponentIndex)
			{
				if (StateData.PreviousInUseComponentFrameBits[ComponentIndex] == true)
				{
					++TotalNumRunningComponents;
				}
			}
		}
#endif // DETAIL_STATS
	}

#if DETAIL_STATS
	SET_DWORD_STAT(STAT_AnimationSharing_NumOnDemands, TotalNumOnDemands);
	SET_DWORD_STAT(STAT_AnimationSharing_NumBlends, TotalNumBlends);

	SET_DWORD_STAT(STAT_AnimationSharing_NumActors, TotalNumActors);
	SET_DWORD_STAT(STAT_AnimationSharing_NumComponent, TotalNumComponents);

	SET_DWORD_STAT(STAT_AnimationSharing_NumBlends, TotalNumBlends);
#endif // DETAIL_STATS

#if CSV_STATS 
	CSV_CUSTOM_STAT_GLOBAL(NumOnDemands, TotalNumOnDemands, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_GLOBAL(NumBlends, TotalNumBlends, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_GLOBAL(NumRunningStates, TotalNumRunningStates, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_GLOBAL(NumRunningComponents, TotalNumRunningComponents, ECsvCustomStatOp::Set);
#endif

}

bool UAnimationSharingManager::RegisterActor(AActor* InActor, FUpdateActorHandle CallbackDelegate)
{
	if (!UAnimationSharingManager::AnimationSharingEnabled())
	{
		return false;
	}

	if (InActor)
	{
		TArray<USkeletalMeshComponent*, TInlineAllocator<1>> OwnedComponents;
		InActor->GetComponents(OwnedComponents);
		checkf(OwnedComponents.Num(), TEXT("No SkeletalMeshComponents found in actor!"));

		const USkeleton* UsedSkeleton = [&OwnedComponents]()
		{
			const USkeleton* CurrentSkeleton = nullptr;
			for (USkeletalMeshComponent* SkeletalMeshComponent : OwnedComponents)
			{
				const USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
				const USkeleton* Skeleton = Mesh->GetSkeleton();

				if (CurrentSkeleton == nullptr)
				{
					CurrentSkeleton = Skeleton;
				}
				else if (CurrentSkeleton != Skeleton)
				{
					if (!CurrentSkeleton->IsCompatibleMesh(Mesh))
					{
						checkf(false, TEXT("Multiple different skeletons within same actor"));
					}
				}
			}

			return CurrentSkeleton;
		}();

		return RegisterActorWithSkeleton(InActor, UsedSkeleton, CallbackDelegate);
	}

	return false;
}

bool UAnimationSharingManager::RegisterActorWithSkeleton(AActor* InActor, const USkeleton* SharingSkeleton, FUpdateActorHandle CallbackDelegate)
{
	if (!UAnimationSharingManager::AnimationSharingEnabled())
	{
		return false;
	}

	const uint32 InstanceIdx = [this, SharingSkeleton]() -> uint32
	{
		uint32 ArrayIndex = Skeletons.IndexOfByPredicate([SharingSkeleton](const USkeleton* Skeleton)
		{
			return (Skeleton == SharingSkeleton);
		});
		return ArrayIndex;
	}();

	if (InstanceIdx != INDEX_NONE)
	{
		TArray<USkeletalMeshComponent*, TInlineAllocator<1>> OwnedComponents;
		InActor->GetComponents(OwnedComponents);
		checkf(OwnedComponents.Num(), TEXT("No SkeletalMeshComponents found in actor!"));

		UAnimSharingInstance* Data = PerSkeletonData[InstanceIdx];
		if (Data->AnimSharingManager != nullptr)
		{
			// Register the actor
			const int32 ActorIndex = Data->RegisteredActors.Add(InActor);

			FPerActorData& ActorData = Data->PerActorData.AddZeroed_GetRef();
			ActorData.BlendInstanceIndex = ActorData.OnDemandInstanceIndex = ActorData.AdditiveInstanceIndex = INDEX_NONE;
			ActorData.SignificanceValue = Data->SignificanceManager->GetSignificance(InActor);
			ActorData.UpdateActorHandleDelegate = CallbackDelegate;

			bool bShouldProcess = true;
			ActorData.CurrentState = ActorData.PreviousState = Data->DetermineStateForActor(ActorIndex, bShouldProcess);

			for (USkeletalMeshComponent* Component : OwnedComponents)
			{
				FPerComponentData& ComponentData = Data->PerComponentData.AddZeroed_GetRef();
				ComponentData.ActorIndex = ActorIndex;
				ComponentData.WeakComponent = Component;

				Component->SetComponentTickEnabled(false);
				Component->PrimaryComponentTick.bCanEverTick = false;
				Component->bIgnoreLeaderPoseComponentLOD = true;
				
				ActorData.ComponentIndices.Add(Data->PerComponentData.Num() - 1);
			}

			Data->SetupFollowerComponent(ActorData.CurrentState, ActorIndex);

			if (Data->PerStateData[ActorData.CurrentState].bIsOnDemand && ActorData.OnDemandInstanceIndex != INDEX_NONE)
			{
				// We will have setup an on-demand instance so we need to kick it off here before we next tick
				Data->OnDemandInstances[ActorData.OnDemandInstanceIndex].bActive = true;
				Data->OnDemandInstances[ActorData.OnDemandInstanceIndex].StartTime = Data->WorldTime;
			}

			// for the actor handle use the stored skeleton id not the current skeleton index as this can change if setups are removed and added
			const int32 ActorHandle = CreateActorHandle(IntCastChecked<uint8>(Data->SkeletonID), ActorIndex);

			ActorData.UpdateActorHandleDelegate.ExecuteIfBound(ActorHandle);

			return true;
		}
	}

	return false;
}

bool UAnimationSharingManager::RegisterActorWithSkeletonBP(AActor* InActor, const USkeleton* SharingSkeleton)
{
	return RegisterActorWithSkeleton(InActor, SharingSkeleton, FUpdateActorHandle::CreateLambda([](int32 A) {}));
}

bool UAnimationSharingManager::UnregisterActor(AActor* InActor)
{
	if (!UAnimationSharingManager::AnimationSharingEnabled())
	{
		return false;
	}

	int32 UnregisterCounter = 0;
	for (int32 SkeletonIndex = 0; SkeletonIndex < PerSkeletonData.Num(); ++SkeletonIndex)
	{
		UAnimSharingInstance* SkeletonData = PerSkeletonData[SkeletonIndex];
		const int32 ActorIndex = SkeletonData->RegisteredActors.IndexOfByKey(InActor);	

		if (ActorIndex != INDEX_NONE )
		{
			const FPerActorData& ActorData = SkeletonData->PerActorData[ActorIndex];

			const bool bNeedsSwap = SkeletonData->PerActorData.Num() > 1 && ActorIndex != SkeletonData->PerActorData.Num() - 1;

			for (int32 ComponentIndex : ActorData.ComponentIndices)
			{
				if (USkeletalMeshComponent* Component = SkeletonData->GetComponent(SkeletonData->PerComponentData[ComponentIndex]))
				{
					Component->SetLeaderPoseComponent(nullptr, true);
					Component->PrimaryComponentTick.bCanEverTick = true;
					Component->SetComponentTickEnabled(true);
				}
				SkeletonData->RemoveComponent(ComponentIndex);
			}

			const int32 SwapIndex = SkeletonData->PerActorData.Num() - 1;

			// Remove actor index from any blend instances
			for (FBlendInstance& Instance : SkeletonData->BlendInstances)
			{
				Instance.ActorIndices.Remove(ActorIndex);

				// If we are swapping and the actor we are swapping with is part of the instance make sure we update the actor index
				const uint32 SwapActorIndex = bNeedsSwap ? Instance.ActorIndices.IndexOfByKey(SwapIndex) : INDEX_NONE;
				if (SwapActorIndex != INDEX_NONE)
				{
					Instance.ActorIndices[SwapActorIndex] = ActorIndex;
				}
			}

			// Remove actor index from any running on demand instances
			for (FOnDemandInstance& Instance : SkeletonData->OnDemandInstances)
			{
				Instance.ActorIndices.Remove(ActorIndex);

				// If we are swapping and the actor we are swapping with is part of the instance make sure we update the actor index
				const uint32 SwapActorIndex = bNeedsSwap ? Instance.ActorIndices.IndexOfByKey(SwapIndex) : INDEX_NONE;
				if (SwapActorIndex != INDEX_NONE)
				{
					Instance.ActorIndices[SwapActorIndex] = ActorIndex;
				}
			}

			// Remove actor index from any additive instances
			for (FAdditiveInstance& Instance : SkeletonData->AdditiveInstances)
			{
				if (Instance.ActorIndex == ActorIndex)
				{
					Instance.ActorIndex = INDEX_NONE;
				}
				else if (bNeedsSwap && Instance.ActorIndex == SwapIndex)
				{
					Instance.ActorIndex = ActorIndex;
				}
			}

			if (bNeedsSwap)
			{
				// Swap actor index for all components which are part of the actor we are swapping with
				for (uint32 ComponentIndex : SkeletonData->PerActorData[SwapIndex].ComponentIndices)
				{
					SkeletonData->PerComponentData[ComponentIndex].ActorIndex = ActorIndex;
				}

				// Make sure we update the handle on the swapped actor
				SkeletonData->PerActorData[SwapIndex].UpdateActorHandleDelegate.ExecuteIfBound(CreateActorHandle(IntCastChecked<uint8>(SkeletonData->SkeletonID), ActorIndex));
			}			

			SkeletonData->PerActorData.RemoveAtSwap(ActorIndex, EAllowShrinking::No);
			SkeletonData->RegisteredActors.RemoveAtSwap(ActorIndex, EAllowShrinking::No);

			UnregisterCounter++;
		}
	}

	// Return whether or not the actor was unregistered from at least one of the Anim Sharing Instances
	return UnregisterCounter > 0;
}

void UAnimationSharingManager::UpdateSignificanceForActorHandle(uint32 InHandle, float InValue)
{
	// Retrieve actor
	if (FPerActorData* ActorData = GetActorDataByHandle(InHandle))
	{
		ActorData->SignificanceValue = InValue;
	}
}

FPerActorData* UAnimationSharingManager::GetActorDataByHandle(uint32 InHandle)
{
	FPerActorData* ActorDataPtr = nullptr;

	if (InHandle == InvalidActorHandle)
	{
		return ActorDataPtr;
	}

	uint8 SkeletonIndex = GetSkeletonIndexFromHandle(InHandle);

	if (SkeletonIndex == MaxSkeletonIndex)
	{
		return ActorDataPtr;
	}

	uint32 ActorIndex = GetActorIndexFromHandle(InHandle);
	if (PerSkeletonData.IsValidIndex(SkeletonIndex))
	{
		if (PerSkeletonData[SkeletonIndex]->PerActorData.IsValidIndex(ActorIndex))
		{
			ActorDataPtr = &PerSkeletonData[SkeletonIndex]->PerActorData[ActorIndex];
		}
	}

	return ActorDataPtr;
}

void UAnimationSharingManager::ClearActorData()
{
	UnregisterAllActors();

	for (UAnimSharingInstance* Data : PerSkeletonData)
	{		
		Data->BlendInstances.Empty();
		Data->OnDemandInstances.Empty();
	}
}

void UAnimationSharingManager::UnregisterAllActors()
{
	for (UAnimSharingInstance* Data : PerSkeletonData)
	{
		Data->UnregisterAllActors();
	}	
}

void UAnimationSharingManager::SetLeaderComponentsVisibility(bool bVisible)
{
	for (UAnimSharingInstance* Data : PerSkeletonData)
	{
		for (FPerStateData& StateData : Data->PerStateData)
		{
			for (USkeletalMeshComponent* Component : StateData.Components)
			{
				Component->SetVisibility(bVisible);
			}
		}

		for (FTransitionBlendInstance* Instance : Data->BlendInstanceStack.AvailableInstances)
		{
			if (USceneComponent* Component = Instance->GetComponent())
			{
				Component->SetVisibility(bVisible);
			}
		}

		for (FTransitionBlendInstance* Instance : Data->BlendInstanceStack.InUseInstances)
		{
			if (USceneComponent* Component = Instance->GetComponent())
			{
				Component->SetVisibility(bVisible);
			}
		}
	}
}

bool UAnimationSharingManager::AnimationSharingEnabled()
{
	return GAnimationSharingEnabled == 1;
}

bool UAnimationSharingManager::CreateAnimationSharingManager(UObject* WorldContextObject, const UAnimationSharingSetup* Setup)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		return FAnimSharingModule::CreateAnimationSharingManager(World, Setup);
	}

	return false;
}

void UAnimationSharingManager::SetDebugMaterial(USkeletalMeshComponent* Component, uint8 State)
{
#if DEBUG_MATERIALS 
	if (GAnimationSharingDebugging >= 1 && GAnimationSharingDisableDebugMaterials == 0 && DebugMaterials.IsValidIndex(State))
	{
		const int32 NumMaterials = Component->GetNumMaterials();
		for (int32 Index = 0; Index < NumMaterials; ++Index)
		{
			Component->SetMaterial(Index, DebugMaterials[State]);
		}
	}
#endif
}

void UAnimationSharingManager::SetDebugMaterialForActor(UAnimSharingInstance* Data, uint32 ActorIndex, uint8 State)
{
#if DEBUG_MATERIALS 
	for (uint32 ComponentIndex : Data->PerActorData[ActorIndex].ComponentIndices)
	{
		if (USkeletalMeshComponent* Component = Data->GetComponent(Data->PerComponentData[ComponentIndex]))
		{
			SetDebugMaterial(Component, State);
		}
	}
#endif
}

#if WITH_EDITOR
FName UAnimationSharingManager::GetPlatformName()
{
	const FString PlatformString = CVarAnimSharing_PreviewScalabilityPlatform.GetValueOnAnyThread();
	if (PlatformString.IsEmpty())
	{
		return FName(FPlatformProperties::IniPlatformName());
	}

	FName PlatformNameFromString(*PlatformString);
	return PlatformNameFromString;
}
#endif

USkeletalMeshComponent* UAnimSharingInstance::GetComponent(const FPerComponentData& Component)
{
	if (Component.WeakComponent.IsValid())
	{
		return Component.WeakComponent.Get();
	}
	
	UE_LOGF(LogAnimationSharing, Warning, "Stale SkeletalMeshComponent\n\tActor Index: %i\n\tActor: %ls", Component.ActorIndex, RegisteredActors[Component.ActorIndex].Get() ? *RegisteredActors[Component.ActorIndex]->GetName() : TEXT("None"));

	return nullptr;
}

void UAnimSharingInstance::BeginDestroy()
{
	Super::BeginDestroy();

	UnregisterAllActors();

	if (UWorld* World = GetWorld())
	{
		if (SharingActor)
		{
			World->DestroyActor(SharingActor);
		}
	}

	PerStateData.Empty();
	StateProcessor = nullptr;
	StateEnum = nullptr;
	OwnerSetup = nullptr;
	BlendInstances.Empty();
	OnDemandInstances.Empty();
	AdditiveInstances.Empty();
	UsedAnimationSequences.Empty();
	SignificanceManager = nullptr;
	AnimSharingManager = nullptr;
}

void UAnimSharingInstance::UnregisterAllActors()
{
	for (int32 ActorIndex = 0; ActorIndex < RegisteredActors.Num(); ++ActorIndex)
	{
		AActor* RegisteredActor = RegisteredActors[ActorIndex];
		if (RegisteredActor)
		{
			FPerActorData& ActorData = PerActorData[ActorIndex];
			for (int32 ComponentIndex : ActorData.ComponentIndices)
			{
				if (USkeletalMeshComponent* Component = GetComponent(PerComponentData[ComponentIndex]))
				{
					Component->SetLeaderPoseComponent(nullptr, true);
					Component->PrimaryComponentTick.bCanEverTick = true;
					Component->SetComponentTickEnabled(true);
					Component->bRecentlyRendered = false;
				}
			}
			ActorData.ComponentIndices.Empty();

			ActorData.UpdateActorHandleDelegate.ExecuteIfBound(UAnimationSharingManager::GetInvalidActorHandle());
		}
	}

	PerActorData.Empty();
	PerComponentData.Empty();
	RegisteredActors.Empty();
}

uint8 UAnimSharingInstance::DetermineStateForActor(uint32 ActorIndex, bool& bShouldProcess)
{
	const FPerActorData& ActorData = PerActorData[ActorIndex];
	int32 State = 0;
	if (bNativeStateProcessor)
	{
		StateProcessor->ProcessActorState_Implementation(State, RegisteredActors[ActorIndex], ActorData.CurrentState, ActorData.OnDemandInstanceIndex != INDEX_NONE ? OnDemandInstances[ActorData.OnDemandInstanceIndex].State : INDEX_NONE, bShouldProcess);
	}
	else
	{
		StateProcessor->ProcessActorState(State, RegisteredActors[ActorIndex], ActorData.CurrentState, ActorData.OnDemandInstanceIndex != INDEX_NONE ? OnDemandInstances[ActorData.OnDemandInstanceIndex].State : INDEX_NONE, bShouldProcess);
	}
	
	return FMath::Max((uint8)0, IntCastChecked<uint8>(State));
}

bool UAnimSharingInstance::Setup(UAnimationSharingManager* AnimationSharingManager, const FPerSkeletonAnimationSharingSetup& SkeletonSetup, const FAnimationSharingScalability* InScalabilitySettings, uint32 InSkeletonID)
{
	USkeletalMesh* SkeletalMesh = SkeletonSetup.SkeletalMesh;
	/** Retrieve the state processor to use */
	if (UAnimationSharingStateProcessor* Processor = SkeletonSetup.StateProcessorClass.GetDefaultObject())
	{
		StateProcessor = Processor;
		bNativeStateProcessor = SkeletonSetup.StateProcessorClass->HasAnyClassFlags(CLASS_Native);
	}

	bool bErrors = false;

	if (SkeletalMesh && StateProcessor)
	{
		SkeletonID = InSkeletonID;

		SkeletalMeshBounds = SkeletalMesh->GetBounds().BoxExtent * 2;
		ScalabilitySettings = *InScalabilitySettings;
		StateEnum = StateProcessor->GetAnimationStateEnum();
		const uint32 NumStates = StateEnum->NumEnums();
		PerStateData.AddDefaulted(NumStates);

		UWorld* World = GetWorld();
		SharingActor = World->SpawnActor<AActor>();
		// Make sure the actor stays around when scrubbing through replays, states will be updated correctly in next tick 
		SharingActor->bReplayRewindable = true;
		SignificanceManager = USignificanceManager::Get<USignificanceManager>(World);
		AnimSharingManager = AnimationSharingManager;

		/** Create runtime data structures for unique animation states */
		NumSetups = 0;
		for (const FAnimationStateEntry& StateEntry : SkeletonSetup.AnimationStates)
		{
			const uint8 StateValue = StateEntry.State;
			const uint32 StateIndex = StateEnum->GetIndexByValue(StateValue);

			if (!PerStateData.FindByPredicate([StateValue](const FPerStateData& State) { return State.StateEnumValue == StateValue; }))
			{
				FPerStateData& StateData = PerStateData[StateIndex];
				StateData.StateEnumValue = StateValue;
				SetupState(StateData, StateEntry, SkeletalMesh, SkeletonSetup, SkeletonID);

				// Make sure we have at least one component set up
				if (StateData.Components.Num() == 0)
				{
					UE_LOGF(LogAnimationSharing, Error, "No Components available for State %ls", *StateEnum->GetDisplayNameTextByValue(StateValue).ToString());
					bErrors = true;
				}
			}
			else
			{
				UE_LOGF(LogAnimationSharing, Error, "Duplicate entries in Animation Setup for State %ls", *StateEnum->GetDisplayNameTextByValue(StateValue).ToString());
				bErrors = true;
			}
		}

		if (bErrors)
		{
			PerStateData.Empty();
		}

		/** Setup blend actors, if enabled*/
		if (!bErrors && ScalabilitySettings.UseBlendTransitions.Default)
		{
			const uint32 TotalNumberOfBlendActorsRequired = ScalabilitySettings.MaximumNumberConcurrentBlends.Default;
			const float ZOffset = static_cast<float>((double)SkeletonID * SkeletalMeshBounds.Z * 2.0);
			for (uint32 BlendIndex = 0; BlendIndex < TotalNumberOfBlendActorsRequired; ++BlendIndex)
			{
				const FVector SpawnLocation(BlendIndex * SkeletalMeshBounds.X, 0.f, ZOffset + SkeletalMeshBounds.Z);
				const FName BlendComponentName(*(SkeletalMesh->GetName() + TEXT("_BlendComponent") + FString::FromInt(BlendIndex)));
				USkeletalMeshComponent* BlendComponent = NewObject<USkeletalMeshComponent>(SharingActor, BlendComponentName);
				BlendComponent->RegisterComponent();
				BlendComponent->SetRelativeLocation(SpawnLocation);
				BlendComponent->SetSkeletalMesh(SkeletalMesh);
				BlendComponent->SetVisibility(GLeaderComponentsVisible == 1);
				BlendComponent->bEnableMaterialParameterCaching = SkeletonSetup.bEnableMaterialParameterCaching;

				BlendComponent->bPropagateCurvesToFollowers = ScalabilitySettings.RequireCurvesDuringBlends.Default;

				BlendComponent->PrimaryComponentTick.AddPrerequisite(AnimSharingManager, AnimSharingManager->GetTickFunction());

				FTransitionBlendInstance* BlendActor = new FTransitionBlendInstance();
				BlendActor->Initialise(BlendComponent, SkeletonSetup.BlendAnimBlueprint.Get());
				BlendInstanceStack.AddInstance(BlendActor);
			}
		}
	}
	else
	{
		UE_LOGF(LogAnimationSharing, Error, "Invalid Skeletal Mesh or State Processing Class");
		bErrors = true;
	}

	return !bErrors;
}

void UAnimSharingInstance::SetupState(FPerStateData& StateData, const FAnimationStateEntry& StateEntry, USkeletalMesh* SkeletalMesh, const FPerSkeletonAnimationSharingSetup& SkeletonSetup, uint32 InSkeletonID)
{
	/** Used for placing components into rows / columns at origin for debugging purposes */
	const float ZOffset = static_cast<float>((double)SkeletonID * SkeletalMeshBounds.Z * 2.0);

	/** Setup overall data and flags */
	StateData.bIsOnDemand = StateEntry.bOnDemand;
	StateData.bIsAdditive = StateEntry.bAdditive;
	StateData.AdditiveAnimationSequence = (StateEntry.bAdditive && StateEntry.AnimationSetups.IsValidIndex(0)) ? ToRawPtr(StateEntry.AnimationSetups[0].AnimSequence) : nullptr;

	/** Keep hard reference to animation sequence */
	if (StateData.AdditiveAnimationSequence)
	{
		UsedAnimationSequences.Add(StateData.AdditiveAnimationSequence);
	}

	StateData.BlendTime = StateEntry.BlendTime;	
	StateData.bReturnToPreviousState = StateEntry.bReturnToPreviousState;
	StateData.bShouldForwardToState = StateEntry.bSetNextState;
	StateData.ForwardStateValue = StateEntry.NextState;
	StateData.bShouldRandomizeOnDemandAnims = StateEntry.bRandomizeOnDemandAnims;

	int32 MaximumNumberOfConcurrentInstances = StateEntry.MaximumNumberOfConcurrentInstances.Default;
#if WITH_EDITOR
	const FName PlatformName = UAnimationSharingManager::GetPlatformName();
	MaximumNumberOfConcurrentInstances = StateEntry.MaximumNumberOfConcurrentInstances.GetValueForPlatform(PlatformName);
#endif

	/** Ensure that we spread our number over the number of enabled setups */
	const int32 NumInstancesPerSetup = [MaximumNumberOfConcurrentInstances, &StateEntry]()
	{
		int32 TotalEnabled = 0;
		for (const FAnimationSetup& AnimationSetup : StateEntry.AnimationSetups)
		{
			bool bEnabled = AnimationSetup.Enabled.Default;
#if WITH_EDITOR
			const FName PlatformName = UAnimationSharingManager::GetPlatformName();
			bEnabled = AnimationSetup.Enabled.GetValueForPlatform(PlatformName);
#endif
			TotalEnabled += bEnabled ? 1 : 0;
		}

		return (TotalEnabled > 0) ? FMath::CeilToInt((float)MaximumNumberOfConcurrentInstances / (float)TotalEnabled) : 0;
	}();

	UWorld* World = GetWorld();
	/** Setup animations used for this state and the number of permutations */
	TArray<USkeletalMeshComponent*>& Components = StateData.Components;
	for (int32 SetupIndex = 0; SetupIndex < StateEntry.AnimationSetups.Num(); ++SetupIndex)
	{
		const FAnimationSetup& AnimationSetup = StateEntry.AnimationSetups[SetupIndex];
		/** User can setup either an AnimBP or AnimationSequence */
		UClass* AnimBPClass = AnimationSetup.AnimBlueprint.Get();
		UAnimSequence* AnimSequence = AnimationSetup.AnimSequence;
		
		if (AnimBPClass == nullptr && AnimSequence == nullptr)
		{
			UE_LOGF(LogAnimationSharing, Error, "Animation setup entry for state %ls without either a valid Animation Blueprint Class or Animation Sequence", StateEnum ? *StateEnum->GetName() : TEXT("None"));
			continue;
		}

		bool bEnabled = AnimationSetup.Enabled.Default;
#if WITH_EDITOR			
		bEnabled = AnimationSetup.Enabled.GetValueForPlatform(PlatformName);
#endif

		/** Only create component if the setup is enabled for this platform and we have a valid animation asset */
		if (bEnabled && (AnimBPClass || AnimSequence))
		{
			int32 NumRandomizedInstances = AnimationSetup.NumRandomizedInstances.Default;
#if WITH_EDITOR			
			NumRandomizedInstances = AnimationSetup.NumRandomizedInstances.GetValueForPlatform(PlatformName);
#endif
			const uint32 NumInstances = StateEntry.bOnDemand ? NumInstancesPerSetup	: FGenericPlatformMath::Max(NumRandomizedInstances, 1);
			for (uint32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
			{
				if (!StateData.bIsAdditive)
				{
					const FName StateComponentName(*(SkeletalMesh->GetName() + TEXT("_") + StateEnum->GetNameStringByIndex(StateEntry.State) + FString::FromInt(SetupIndex) + FString::FromInt(InstanceIndex)));
					USkeletalMeshComponent* Component = NewObject<USkeletalMeshComponent>(SharingActor, StateComponentName);
					Component->RegisterComponent();
					/** Arrange component in correct row / column */
					Component->SetRelativeLocation(FVector(NumSetups * SkeletalMeshBounds.X, 0.f, ZOffset));
					/** Set shared skeletal mesh */
					Component->SetSkeletalMesh(SkeletalMesh);
					Component->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
					// Force component to always run at LOD0, to ensure all follower-components will always be at the same or higher LOD level for bone transforms
					Component->SetForcedLOD(1);
					Component->SetVisibility(GLeaderComponentsVisible == 1);
					Component->bPropagateCurvesToFollowers = StateEntry.bRequiresCurves;
					Component->bEnableMaterialParameterCaching = SkeletonSetup.bEnableMaterialParameterCaching;

					if (AnimBPClass != nullptr && AnimSequence != nullptr && !StateEntry.bOnDemand)
					{
						Component->SetAnimInstanceClass(AnimBPClass);
						if (UAnimSharingStateInstance* AnimInstance = Cast<UAnimSharingStateInstance>(Component->GetAnimInstance()))
						{
							AnimInstance->AnimationToPlay = AnimSequence;
							if (InstanceIndex > 0)
							{
								const float Steps = (AnimSequence->GetPlayLength() * 0.9f) / static_cast<float>(NumInstances);
								const float StartTimeOffset = Steps * static_cast<float>(InstanceIndex);
								AnimInstance->PermutationTimeOffset = StartTimeOffset;
							}

							AnimInstance->PlayRate = StateData.bIsOnDemand ? 0.f : 1.0f;

							AnimInstance->Instance = this;
							AnimInstance->StateIndex = StateEntry.State;
							AnimInstance->ComponentIndex = IntCastChecked<uint8>(Components.Num());

							/** Set the current animation length */
							StateData.AnimationLengths.Add(AnimSequence->GetPlayLength());

							AnimInstance->InitializeAnimation(false);
						}
					}
					else if (AnimSequence != nullptr)
					{
						Component->PlayAnimation(AnimSequence, true);

						/** If this is an on-demand state we pause the animation as we'll want to start it from the beginning anytime we start an on-demand instance */
						if (StateData.bIsOnDemand)
						{
							Component->Stop();
						}
						else
						{
							if (InstanceIndex > 0)
							{
								const float Steps = (AnimSequence->GetPlayLength() * 0.9f) / static_cast<float>(NumInstances);
								const float StartTimeOffset = Steps * static_cast<float>(InstanceIndex);
								Component->SetPosition(StartTimeOffset, false);
							}
						}

						/** Set the current animation length length */
						StateData.AnimationLengths.Add(AnimSequence->GetPlayLength());
					}

					/** Set material to red to indicate that it's not in use*/
					UAnimationSharingManager::SetDebugMaterial(Component, 0);

					Component->PrimaryComponentTick.AddPrerequisite(AnimSharingManager, AnimSharingManager->GetTickFunction());
					Components.Add(Component);
				}
				else
				{	
					const FVector SpawnLocation(FVector(NumSetups * SkeletalMeshBounds.X, 0.f, ZOffset));
					const FName AdditiveComponentName(*(SkeletalMesh->GetName() + TEXT("_") + StateEnum->GetNameStringByIndex(StateEntry.State) + FString::FromInt(InstanceIndex)));
					USkeletalMeshComponent* AdditiveComponent = NewObject<USkeletalMeshComponent>(SharingActor, AdditiveComponentName);
					AdditiveComponent->RegisterComponent();
					AdditiveComponent->SetRelativeLocation(SpawnLocation);
					AdditiveComponent->SetSkeletalMesh(SkeletalMesh);
					// Force component to always run at LOD0, to ensure all follower-components will always be at the same or higher LOD level for bone transforms
					AdditiveComponent->SetForcedLOD(1);
					AdditiveComponent->SetVisibility(GLeaderComponentsVisible == 1);
					AdditiveComponent->bEnableMaterialParameterCaching = SkeletonSetup.bEnableMaterialParameterCaching;

					AdditiveComponent->PrimaryComponentTick.AddPrerequisite(AnimSharingManager, AnimSharingManager->GetTickFunction());

					FAdditiveAnimationInstance* AdditiveInstance = new FAdditiveAnimationInstance();
					AdditiveInstance->Initialise(AdditiveComponent, SkeletonSetup.AdditiveAnimBlueprint.Get());
					StateData.AdditiveInstanceStack.AddInstance(AdditiveInstance);
					
					/** Set the current animation length  */
					StateData.AnimationLengths.Add(AnimSequence->GetPlayLength());
					Components.Add(AdditiveComponent);
				}
				
				++NumSetups;
			}
		}
	}

	float TotalLength = 0.f;
	for (float Length : StateData.AnimationLengths)
	{
		TotalLength += Length;
	}
	const float AverageLength = (StateData.AnimationLengths.Num() > 0) ? TotalLength / FMath::Min((float)StateData.AnimationLengths.Num(), 1.f) : 0.f;
	StateData.WiggleTime = AverageLength * StateEntry.WiggleTimePercentage;

	/** Randomizes the order of Components so we actually hit different animations when running on demand */
	if (StateData.bIsOnDemand && !StateData.bIsAdditive && StateEntry.AnimationSetups.Num() > 1)
	{	
		TArray<USkeletalMeshComponent*> RandomizedComponents;
		while (Components.Num() > 0)
		{
			const int32 RandomIndex = FMath::RandRange(0, Components.Num() - 1);
			RandomizedComponents.Add(Components[RandomIndex]);
			Components.RemoveAt(RandomIndex, 1);
		}

		Components = RandomizedComponents;
	}

	/** Initialize component (previous frame) usage flags */
	StateData.InUseComponentFrameBits.Init(false, Components.Num());
	/** This should enforce turning off the components tick during the first frame */
	StateData.PreviousInUseComponentFrameBits.Init(true, Components.Num());

	StateData.FollowerTickRequiredFrameBits.Init(false, Components.Num());
}

#if UE_BUILD_DEVELOPMENT
void UAnimSharingInstance::GetAnimAssetNameAndData(const uint32 StateIdx, const uint32 CompIdx, FString& AnimAssetName, float& AnimTime, float& AnimLength) const
{
	if (PerStateData[StateIdx].Components.IsValidIndex(CompIdx))
	{
		USkeletalMeshComponent* Comp = PerStateData[StateIdx].Components[CompIdx];
		const EAnimationMode::Type AnimMode = Comp->GetAnimationMode();
		switch (AnimMode)
		{
			case EAnimationMode::AnimationSingleNode:
			{
				if (UAnimSingleNodeInstance* SingleNodeAnimInstance = Comp->GetSingleNodeInstance())
				{
					if (UAnimationAsset* AnimAsset = SingleNodeAnimInstance->GetAnimationAsset())
					{
						AnimAssetName = AnimAsset->GetName();
					}
					AnimTime = SingleNodeAnimInstance->GetCurrentTime();
					AnimLength = SingleNodeAnimInstance->GetLength();
				}
			}
			break;
			
			case EAnimationMode::AnimationBlueprint:
			{
				if (UAnimInstance* AnimInstance = Comp->GetAnimInstance())
				{
					AnimAssetName = AnimInstance->GetName();

					static FName AnimGraphName(TEXT("AnimGraph"));
					TArray<const FAnimNode_AssetPlayerRelevancyBase*> PlayerArray = AnimInstance->GetInstanceRelevantAssetPlayers(AnimGraphName);
					if (PlayerArray.Num() > 0)
					{
						if (UAnimationAsset* AnimAsset = PlayerArray[0]->GetAnimAsset())
						{
							AnimAssetName = AnimAsset->GetName();
						}
						AnimTime = PlayerArray[0]->GetCurrentAssetTime();
						AnimLength = PlayerArray[0]->GetCurrentAssetLength();
					}
				}
			}
			break;

			default:
				break;
		}
	}
}

FString UAnimSharingInstance::GetDebugStateString(uint8 State, const uint32 CompIdx, uint32 OnDemandInstanceIndex) const
{
	if (OnDemandInstanceIndex != INDEX_NONE && OnDemandInstances.IsValidIndex(OnDemandInstanceIndex))
	{
		const FOnDemandInstance& OnDemandInstance = OnDemandInstances[OnDemandInstanceIndex];
		const uint32 OnDemandStateIdx = OnDemandInstance.State;
		const uint32 OnDemandCompIdx = OnDemandInstance.UsedPerStateComponentIndex;

		FString OnDemandAnimName;
		float OnDemandAnimTime = 0.0f;
		float OnDemandAnimLength = 0.0f;
		GetAnimAssetNameAndData(OnDemandStateIdx, OnDemandCompIdx, OnDemandAnimName, OnDemandAnimTime, OnDemandAnimLength);

		return FString::Printf(TEXT("On Demand [%i] Anim %s [%.2f / %.2f] {%d}"), OnDemandInstanceIndex, *OnDemandAnimName, OnDemandAnimTime, OnDemandAnimLength, OnDemandCompIdx);
	}
	else
	{
		FString AnimName;
		float AnimTime = 0.0f;
		float AnimLength = 0.0f;
		GetAnimAssetNameAndData(State, CompIdx, AnimName, AnimTime, AnimLength);

		return FString::Printf(TEXT("%s [%.2f / %.2f] {%d}"), *AnimName, AnimTime, AnimLength, CompIdx);
	}
}
#endif // #if UE_BUILD_DEVELOPMENT

void UAnimSharingInstance::TickDebugInformation()
{
#if !UE_BUILD_SHIPPING
#if UE_BUILD_DEVELOPMENT
	if (GLeaderComponentsVisible && GAnimationSharingDebugging >= 2)
	{
		for (const FPerStateData& StateData : PerStateData)
		{
			for (int32 Index = 0; Index < StateData.InUseComponentFrameBits.Num(); ++Index)
			{
				const FString ComponentString = FString::Printf(TEXT("[%s]\n In Use %s - Required %s"), *StateEnum->GetDisplayNameTextByValue(StateData.StateEnumValue).ToString(), StateData.InUseComponentFrameBits[Index] ? TEXT("True") : TEXT("False"), StateData.FollowerTickRequiredFrameBits[Index] ? TEXT("True") : TEXT("False"));
				DrawDebugString(GetWorld(), StateData.Components[Index]->GetComponentLocation() + FVector(0,0,StateData.Components[Index]->Bounds.BoxExtent.Z), ComponentString, nullptr, FColor::White, 0.016f, false);
			}
		}
	}
#endif // UE_BUILD_DEVELOPMENT
	
	for (int32 ActorIndex = 0; ActorIndex < RegisteredActors.Num(); ++ActorIndex)
	{
		// Non-const for DrawDebugString
		AActor* Actor = RegisteredActors[ActorIndex];
		if (Actor)
		{
			const FPerActorData& ActorData = PerActorData[ActorIndex];

#if UE_BUILD_DEVELOPMENT
			const uint8 State = ActorData.CurrentState;

			const FColor DebugColor = [&ActorData]()
			{
				const uint32 BlendInstanceIndex = ActorData.BlendInstanceIndex;
				const uint32 DemandInstanceIndex = ActorData.OnDemandInstanceIndex;

				/** Colors match debug material colors */
				if (ActorData.bBlending && BlendInstanceIndex != INDEX_NONE)
				{
					return FColor::Blue;
				}
				else if (ActorData.bRunningOnDemand && DemandInstanceIndex != INDEX_NONE)
				{
					return FColor::Red;
				}

				return FColor::Green;
			}();

			const FString StateString = [this, &ActorData, State]() -> FString
			{
				/** Check whether or not we are currently blending */
				const uint32 BlendInstanceIndex = ActorData.BlendInstanceIndex;
				if (BlendInstanceIndex != INDEX_NONE && BlendInstances.IsValidIndex(BlendInstanceIndex))
				{
					const float TimeLeft = BlendInstances[BlendInstanceIndex].BlendTime - static_cast<float>(GetWorld()->GetTimeSeconds() - BlendInstances[BlendInstanceIndex].EndTime);
					return FString::Printf(TEXT("Blending states - %s to %s [%1.3f] (%i)"), *StateEnum->GetDisplayNameTextByValue(BlendInstances[BlendInstanceIndex].StateFrom).ToString(), *StateEnum->GetDisplayNameTextByValue(BlendInstances[BlendInstanceIndex].StateTo).ToString(), TimeLeft, ActorData.BlendInstanceIndex);
				}

				/** Check if we are part of an on-demand instance */ 
				const uint32 DemandInstanceIndex = ActorData.OnDemandInstanceIndex;
				if (DemandInstanceIndex != INDEX_NONE && OnDemandInstances.IsValidIndex(DemandInstanceIndex))
				{
					return FString::Printf(TEXT("On demand state - %s [%i]"), *StateEnum->GetDisplayNameTextByValue(State).ToString(), ActorData.OnDemandInstanceIndex);
				}

				/** Otherwise we should just be part of a state */
				return FString::Printf(TEXT("State - %s %1.2f"), *StateEnum->GetDisplayNameTextByValue(State).ToString(), ActorData.SignificanceValue);
			}();

			const FString FullStateString = [this, &ActorData, State]()
			{
				/** Check whether or not we are currently blending */
				const uint32 BlendInstanceIndex = ActorData.BlendInstanceIndex;
				if (BlendInstanceIndex != INDEX_NONE && BlendInstances.IsValidIndex(BlendInstanceIndex))
				{
					const FBlendInstance& BlendInstance = BlendInstances[BlendInstanceIndex];
					uint8 StateFrom = BlendInstance.StateFrom;
					uint8 StateTo = BlendInstance.StateTo;
					const float TimeLeft = BlendInstances[BlendInstanceIndex].EndTime - static_cast<float>(GetWorld()->GetTimeSeconds());

					return FString::Printf(TEXT("Blending states - %s to %s [%1.3f] (%i)\n  From: %s\n  To: %s"),
						*StateEnum->GetDisplayNameTextByValue(StateFrom).ToString(),
						*StateEnum->GetDisplayNameTextByValue(StateTo).ToString(),
						TimeLeft, ActorData.BlendInstanceIndex, 
						*GetDebugStateString(StateFrom, BlendInstance.FromPermutationIndex, BlendInstance.FromOnDemandInstanceIndex),
						*GetDebugStateString(StateTo, BlendInstance.ToPermutationIndex, BlendInstance.ToOnDemandInstanceIndex)
					);
				}
				else
				{
					/** Otherwise we should just be part of a state */
					return FString::Printf(TEXT("State - %s Sig %1.2f\n  %s"), *StateEnum->GetDisplayNameTextByValue(State).ToString(),
							ActorData.SignificanceValue, *GetDebugStateString(State, ActorData.PermutationIndex, ActorData.OnDemandInstanceIndex));
				}
			}();

			/** Draw text above AI pawn's head */
			DrawDebugString(GetWorld(), FVector(0.f, 0.f, 100.f), (GAnimationSharingDebugging >= 3) ? FullStateString : StateString, 
				Actor, GAnimationSharingDisableDebugMaterials ? FColor::Green : DebugColor, 0.016f, false, GAnimationSharingDebugFontScale);
#endif
			if (GAnimationSharingDebugging >= 2)
			{
				const FString OnScreenString = FString::Printf(TEXT("%s\n\tState %s [%i]\n\t%s\n\tBlending %i On-Demand %i"), *Actor->GetName(), *StateEnum->GetDisplayNameTextByValue(ActorData.CurrentState).ToString(), ActorData.PermutationIndex, *StateEnum->GetDisplayNameTextByValue(ActorData.PreviousState).ToString(), ActorData.bBlending, ActorData.bRunningOnDemand);

				GEngine->AddOnScreenDebugMessage(1337, 1, FColor::White, OnScreenString);

				USkinnedMeshComponent* LeaderComponent = nullptr;
				if (USkeletalMeshComponent* Component = GetComponent(PerComponentData[ActorData.ComponentIndices[0]]))
				{
					LeaderComponent = Component->LeaderPoseComponent.Get();
				}
#if UE_BUILD_DEVELOPMENT
				if (LeaderComponent != nullptr)
				{
					DrawDebugLine(GetWorld(), Actor->GetActorLocation(), LeaderComponent->GetComponentLocation(), FColor::Magenta);
				}
#endif
			}

			
		}
	}
#endif
}

void UAnimSharingInstance::TickOnDemandInstances()
{
	SCOPE_CYCLE_COUNTER(STAT_AnimationSharing_UpdateOnDemands);
	for (int32 InstanceIndex = 0; InstanceIndex < OnDemandInstances.Num(); ++ InstanceIndex)
	{
		FOnDemandInstance& Instance = OnDemandInstances[InstanceIndex];
		checkf(Instance.bActive, TEXT("Container should be active at this point"));

		// Mark on-demand component as in-use
		SetComponentUsage(true, Instance.State, Instance.UsedPerStateComponentIndex);

		const bool bShouldTick = DoAnyActorsRequireTicking(Instance);
		if (bShouldTick)
		{
			// Mark component to tick
			SetComponentTick(Instance.State, Instance.UsedPerStateComponentIndex);
		}

		// Check and see whether or not the animation has finished
		if (Instance.EndTime <= WorldTime)
		{
			// Set in-use flag to false this should set the component to not tick during the next TickAnimationStates
			SetComponentUsage(false, Instance.State, Instance.UsedPerStateComponentIndex);

#if LOG_STATES 
			UE_LOGF(LogAnimationSharing, Log, "Finished on demand [%i] %ls", InstanceIndex, *StateEnum->GetDisplayNameTextByValue(Instance.State).ToString());
#endif
			auto SetActorState = [this, &Instance](uint32 ActorIndex, uint8 NewState)
			{
				if (Instance.BlendToPermutationIndex != INDEX_NONE)
				{
					SetPermutationFollowerComponent(NewState, ActorIndex, Instance.BlendToPermutationIndex);
				}
				else
				{
					SetupFollowerComponent(NewState, ActorIndex);

					// If we are setting up a follower to an on-demand state that is not in use yet it needs to create a new On Demand Instance which will not be kicked-off yet, so do that directly.
					if (PerStateData[NewState].bIsOnDemand)
					{
						const int32 OnDemandInstanceIndex = PerActorData[ActorIndex].OnDemandInstanceIndex;
						if (OnDemandInstanceIndex != INDEX_NONE)
						{
							FOnDemandInstance& NewOnDemandInstance = OnDemandInstances[OnDemandInstanceIndex];
							if (!NewOnDemandInstance.bActive)
							{
								NewOnDemandInstance.bActive = true;
								NewOnDemandInstance.StartTime = WorldTime;
							}
						}
					}
				}

				// Set actor states
				PerActorData[ActorIndex].PreviousState = PerActorData[ActorIndex].CurrentState;
				PerActorData[ActorIndex].CurrentState = NewState;
			};				
			
			
			// Set the components to their current state animation
			for (uint32 ActorIndex : Instance.ActorIndices)
			{					
				const uint8 CurrentState = PerActorData[ActorIndex].CurrentState;
				// Return to the previous active animation state 
				if (Instance.bReturnToPreviousState)
				{
					//for (uint32 ActorIndex : Instance.ActorIndices)
					{
						// Retrieve previous state for the actor 
						const uint8 PreviousActorState = PerActorData[ActorIndex].PreviousState;
						SetActorState(ActorIndex, PreviousActorState);
#if LOG_STATES 
						UE_LOGF(LogAnimationSharing, Log, "Returning [%i] to %ls", ActorIndex, *StateEnum->GetDisplayNameTextByValue(PreviousActorState).ToString());
#endif
					}
				}
				else if (Instance.ForwardState != (uint8)INDEX_NONE)
				{
					// We could forward it to a different state at this point						
					SetActorState(ActorIndex, Instance.ForwardState);
#if LOG_STATES
					UE_LOGF(LogAnimationSharing, Log, "Forwarding [%i] to %ls", ActorIndex, *StateEnum->GetDisplayNameTextByValue(Instance.ForwardState).ToString());
#endif						
				}
				// Only do this if the state is different than the current on-demand one
				else if (CurrentState != Instance.State)
				{
					// If the new state is not an on-demand one and we are not currently blending, if we are blending the blend will set the final leader component
					if (!PerStateData[CurrentState].bIsOnDemand || !Instance.bBlendActive)
					{
						SetActorState(ActorIndex, CurrentState);

						UAnimationSharingManager::SetDebugMaterialForActor(this, ActorIndex, 1);
#if LOG_STATES 
						UE_LOGF(LogAnimationSharing, Log, "Setting [%i] to %ls", ActorIndex, *StateEnum->GetDisplayNameTextByValue(CurrentState).ToString());
#endif
					}
				}
				else
				{
					// Otherwise what do we do TODO
#if LOG_STATES 
					UE_LOGF(LogAnimationSharing, Log, "TODO-ing [%i]", ActorIndex);
#endif
				}					
			}			

			// Clear out data for each actor part of this instance
			for (uint32 ActorIndex : Instance.ActorIndices)
			{
				const bool bPartOfOtherOnDemand = PerActorData[ActorIndex].OnDemandInstanceIndex != InstanceIndex;
				//ensureMsgf(!bPartOfOtherOnDemand, TEXT("Actor on demand index differs from current instance"));

				PerActorData[ActorIndex].OnDemandInstanceIndex = INDEX_NONE;
				PerActorData[ActorIndex].bRunningOnDemand = false;			
			}

			// Remove this instance as it has finished work
			RemoveOnDemandInstance(InstanceIndex);

			// Decrement index so we don't skip the swapped instance
			--InstanceIndex;
		}
		else if (!Instance.bBlendActive && Instance.StartBlendTime <= WorldTime)
		{
			for (uint32 ActorIndex : Instance.ActorIndices)
			{
				// Whether or not we can/should actually blend
				const bool bShouldBlend = ScalabilitySettings.UseBlendTransitions.Default && PerActorData[ActorIndex].SignificanceValue >= ScalabilitySettings.BlendSignificanceValue.Default;

				// Determine state to blend to
				const uint8 BlendToState = [this, &Instance, bShouldBlend, ActorIndex]() -> uint8
				{
					if (bShouldBlend)
					{
						bool bShouldProcess;
						const uint32 DeterminedState = DetermineStateForActor(ActorIndex, bShouldProcess);
						const uint32 CurrentState = PerActorData[ActorIndex].CurrentState != DeterminedState ? DeterminedState : PerActorData[ActorIndex].CurrentState;

						if (Instance.bReturnToPreviousState)
						{
							// Setup blend from on-demand animation into next state animation
							return PerActorData[ActorIndex].PreviousState;
						}
						else if (Instance.ForwardState != (uint8)INDEX_NONE)
						{
							// Blend into the forward state 
							return Instance.ForwardState;
						}
						else if (PerActorData[ActorIndex].CurrentState != Instance.State)
						{
							// Blend to the actor's current state
							return PerActorData[ActorIndex].CurrentState;
						}
					}
					return INDEX_NONE;					
				}();

				// Try to setup blending
				if (BlendToState != (uint8)INDEX_NONE)
				{
					const uint32 BlendIndex = SetupBlendFromOnDemand(BlendToState, InstanceIndex, ActorIndex);

					if (BlendIndex != INDEX_NONE)
					{
						// TODO what if two actors have a different state they are blending to? --> Store permutation index
						Instance.BlendToPermutationIndex = BlendInstances[BlendIndex].ToPermutationIndex;
						// Make sure to update actor data with running blend instance (index)
						PerActorData[ActorIndex].BlendInstanceIndex = BlendIndex;
#if LOG_STATES 
						UE_LOGF(LogAnimationSharing, Log, "Blending [%i] out from %ls to %ls", ActorIndex, *StateEnum->GetDisplayNameTextByValue(Instance.State).ToString(), *StateEnum->GetDisplayNameTextByValue(BlendToState).ToString());
#endif
					}
				}

				// OR results, some actors could not be blending 
				Instance.bBlendActive |= bShouldBlend;
			}
		}
	}	
}

void UAnimSharingInstance::TickAdditiveInstances()
{
	SCOPE_CYCLE_COUNTER(STAT_AnimationSharing_UpdateAdditives);

	for (int32 InstanceIndex = 0; InstanceIndex < AdditiveInstances.Num(); ++InstanceIndex)
	{
		FAdditiveInstance& Instance = AdditiveInstances[InstanceIndex];
		SetComponentUsage(true, Instance.State, Instance.UsedPerStateComponentIndex);		
		SetComponentTick(Instance.State, Instance.UsedPerStateComponentIndex);

		if (Instance.bActive)
		{
			const float WorldTimeSeconds = static_cast<float>(GetWorld()->GetTimeSeconds());
			if (WorldTimeSeconds >= Instance.EndTime)
			{
				// Finish
				if (PerActorData.IsValidIndex(Instance.ActorIndex))
				{
					PerActorData[Instance.ActorIndex].bRunningAdditive = false;
					PerActorData[Instance.ActorIndex].AdditiveInstanceIndex = INDEX_NONE;

					// Set it to base component on top of the additive animation is playing
					SetLeaderComponentForActor(Instance.ActorIndex, Instance.AdditiveAnimationInstance->GetBaseComponent());
				}
				FreeAdditiveInstance(Instance.State, Instance.AdditiveAnimationInstance);
				RemoveAdditiveInstance(InstanceIndex);
				--InstanceIndex;

				SetComponentUsage(false, Instance.State, Instance.UsedPerStateComponentIndex);
			}
		}
		else
		{
			Instance.bActive = true;
			Instance.AdditiveAnimationInstance->Start();
			if (Instance.ActorIndex != INDEX_NONE)
			{				
				SetLeaderComponentForActor(Instance.ActorIndex, Instance.AdditiveAnimationInstance->GetComponent());
			}
		}
	}
}

void UAnimSharingInstance::TickActorStates()
{
	SCOPE_CYCLE_COUNTER(STAT_AnimationSharing_TickActorStates);
	/** Tick each registered actor's state */
	for (int32 ActorIndex = 0; ActorIndex < RegisteredActors.Num(); ++ActorIndex)
	{
		/** Ensure Actor is still available */
		const AActor* Actor = RegisteredActors[ActorIndex];
		if (Actor)
		{
			FPerActorData& ActorData = PerActorData[ActorIndex];
			checkf(ActorData.ComponentIndices.Num(), TEXT("Registered Actor without SkeletalMeshComponents"));

			// Update actor and component visibility
			ActorData.bRequiresTick = ActorData.SignificanceValue >= ScalabilitySettings.TickSignificanceValue.Default;
			for (int32 ComponentIndex : ActorData.ComponentIndices)
			{
				if (USkeletalMeshComponent* Component = GetComponent(PerComponentData[ComponentIndex]))
				{
					if (Component->GetLastRenderTime() > (WorldTime - 1.f))
					{
						Component->bRecentlyRendered = true;
						ActorData.bRequiresTick = true;
					}
				}
			}

			// Determine current state for Actor
			uint8& PreviousState = ActorData.CurrentState;
			bool bShouldProcess = false;
			const uint8 CurrentState = DetermineStateForActor(ActorIndex, bShouldProcess);

			// Determine whether we should blend according to the scalability settings
			const bool bShouldBlend = ScalabilitySettings.UseBlendTransitions.Default && ActorData.SignificanceValue >= ScalabilitySettings.BlendSignificanceValue.Default;

			/** If the state is different we need to change animations and setup a transition */
			if (CurrentState != PreviousState)
			{
				/** When we are currently running an on-demand state we do not want as state change to impact the current animation */
				const bool bShouldNotProcess = ActorData.bRunningOnDemand && !PerStateData[CurrentState].bIsOnDemand;

				auto UpdateState = [&ActorData, ActorIndex, CurrentState, PreviousState]()
				{
#if LOG_STATES 
					UE_LOGF(LogAnimationSharing, Log, "Setting %i state to %i previous %i | %i", ActorIndex, CurrentState, PreviousState, ActorData.PermutationIndex);
#endif
					ActorData.PreviousState = PreviousState;
					ActorData.CurrentState = CurrentState;
				};

				/** If the processor explicitly outputs that the change in state should not impact behavior, just change state and do nothing */
				if (!bShouldProcess || bShouldNotProcess)
				{
					UpdateState();
#if LOG_STATES 
					UE_LOGF(LogAnimationSharing, Log, "Changing state to %ls from %ls while running on demand %i (%i)", *StateEnum->GetDisplayNameTextByValue(CurrentState).ToString(), *StateEnum->GetDisplayNameTextByValue(ActorData.PreviousState).ToString(), ActorIndex, ActorData.PermutationIndex);
#endif
				}
				/** Play additive animation only if actor isn't already playing one */
				else if (PerStateData[CurrentState].bIsAdditive)
				{
					if (!ActorData.bRunningAdditive)
					{
						USkeletalMeshComponent* BaseComponent = nullptr;
						if (ActorData.bBlending)
						{
							// When running a blend (either between regular states, or in/out of on-demand states), use the blend component as the additive base
							if (BlendInstances.IsValidIndex(ActorData.BlendInstanceIndex))
							{
								BaseComponent = BlendInstances[ActorData.BlendInstanceIndex].TransitionBlendInstance->GetComponent();
							}
							
						}
						else if (ActorData.bRunningOnDemand)
						{
							// When running an on-demand, use the per-state-component used by/running the on demand instance as the additive base
							if(OnDemandInstances.IsValidIndex(ActorData.OnDemandInstanceIndex))
							{
								const FOnDemandInstance& RunningOnDemandInstance = OnDemandInstances[ActorData.OnDemandInstanceIndex];
								BaseComponent = PerStateData[RunningOnDemandInstance.State].Components[RunningOnDemandInstance.UsedPerStateComponentIndex];
							}
						
						}
						else if (PerStateData[PreviousState].Components.IsValidIndex(ActorData.PermutationIndex))
						{
							// When running a regular state, use the permutation component as the additive base 
							BaseComponent = PerStateData[PreviousState].Components[ActorData.PermutationIndex];
						}

						// If we somehow did not manage to find a valid base component, ignore the additive request
						if (BaseComponent)
						{
							const uint32 AdditiveInstanceIndex = SetupAdditiveInstance(CurrentState, BaseComponent);
							if (AdditiveInstanceIndex != INDEX_NONE)
							{
								ActorData.bRunningAdditive = true;
								ActorData.AdditiveInstanceIndex = AdditiveInstanceIndex;
								AdditiveInstances[AdditiveInstanceIndex].ActorIndex = ActorIndex;
							}
						}
						else
						{
							UE_LOGF(LogAnimationSharing, Warning, "Failed to setup additive state %ls for %i. OnDemand %i %i. Blend %i %i. State %i %i", *StateEnum->GetDisplayNameTextByValue(CurrentState).ToString(), ActorIndex, ActorData.bRunningOnDemand, ActorData.OnDemandInstanceIndex, ActorData.bBlending, ActorData.BlendInstanceIndex, PreviousState, ActorData.PermutationIndex);
						}
					}
				}
				/** If we are _already_ running an on-demand instance and the new state is also an on-demand we'll have to blend the new state in*/
				else if (PerStateData[CurrentState].bIsOnDemand)
				{
					/** If the new state is different than the currently running on-demand state, this could happen if we previously only updated the state and not processed it */
					const bool bSetupInstance = (!ActorData.bRunningOnDemand || (ActorData.bRunningOnDemand && OnDemandInstances[ActorData.OnDemandInstanceIndex].State != CurrentState));
					const uint32 OnDemandIndex = bSetupInstance ? SetupOnDemandInstance(CurrentState) : INDEX_NONE;

					if (OnDemandIndex != INDEX_NONE)
					{
						// Make sure we end any current blends
						RemoveFromCurrentBlend(ActorIndex);
						RemoveFromCurrentOnDemand(ActorIndex);

						bool bShouldSwitch = true;
						if (bShouldBlend && !FMath::IsNearlyZero(PerStateData[CurrentState].BlendTime))
						{
							if (ActorData.bRunningOnDemand)
							{
								/** Setup a blend between the current and a new instance*/
								const uint32 BlendInstanceIndex = SetupBlendBetweenOnDemands(IntCastChecked<uint8>(ActorData.OnDemandInstanceIndex), OnDemandIndex, ActorIndex);
								ActorData.BlendInstanceIndex = BlendInstanceIndex;
							}
							else
							{
								/** Setup a blend to an on-demand state/instance */
								const uint32 BlendInstanceIndex = SetupBlendToOnDemand(PreviousState, OnDemandIndex, ActorIndex);
								ActorData.BlendInstanceIndex = BlendInstanceIndex;
							}

							/** Blend was not succesfully set up so switch anyway */
							bShouldSwitch = (ActorData.BlendInstanceIndex == INDEX_NONE);
						}

						if (bShouldSwitch)
						{
							/** Not blending so just switch to other on-demand instance */
							SwitchBetweenOnDemands(ActorData.OnDemandInstanceIndex, OnDemandIndex, ActorIndex);
						}

						/** Add the current actor to the on-demand instance*/
						OnDemandInstances[OnDemandIndex].ActorIndices.Add(ActorIndex);
						/** Also change actor data accordingly*/
						ActorData.OnDemandInstanceIndex = OnDemandIndex;
						ActorData.bRunningOnDemand = true;

						UpdateState();
					}
				}
				/** Otherwise blend towards the new shared state */
				else
				{
					/** If actor is within blending distance setup/reuse a blend instance*/
					bool bShouldSwitch = true;
					if (bShouldBlend)
					{
						const uint32 BlendInstanceIndex = SetupBlend(PreviousState, CurrentState, ActorIndex);
						ActorData.BlendInstanceIndex = BlendInstanceIndex;
						/** Blend was not succesfully set up so switch anyway */
						bShouldSwitch = (ActorData.BlendInstanceIndex == INDEX_NONE);
#if LOG_STATES 
						UE_LOGF(LogAnimationSharing, Log, "Changing state to %ls from %ls with blend %i", *StateEnum->GetDisplayNameTextByValue(CurrentState).ToString(), *StateEnum->GetDisplayNameTextByValue(PreviousState).ToString(), ActorIndex);
#endif
					}
					/** Otherwise just switch it to the new state */
					if (bShouldSwitch)
					{
						SetupFollowerComponent(CurrentState, ActorIndex);
#if LOG_STATES 
						UE_LOGF(LogAnimationSharing, Log, "Changing state to %ls from %ls %i", *StateEnum->GetDisplayNameTextByValue(CurrentState).ToString(), *StateEnum->GetDisplayNameTextByValue(PreviousState).ToString(), ActorIndex);
#endif
					}

					UpdateState();
				}
			}
			/** Flag the currently leader component as in-use */
			else if (!ActorData.bRunningOnDemand && !ActorData.bBlending)
			{
#if LOG_STATES 
				if (!PerStateData[ActorData.CurrentState].Components.IsValidIndex(ActorData.PermutationIndex))
				{
					UE_LOGF(LogAnimationSharing, Log, "Invalid permutation for actor %i is out of range of %i for state %ls by actor %i", ActorData.PermutationIndex, PerStateData[ActorData.CurrentState].Components.Num(), *StateEnum->GetDisplayNameTextByValue(ActorData.CurrentState).ToString(), ActorIndex);
				}
				else if (!PerStateData[ActorData.CurrentState].Components[ActorData.PermutationIndex]->IsComponentTickEnabled())
				{
					UE_LOGF(LogAnimationSharing, Log, "Component not active %i for state %ls by actor %i", ActorData.PermutationIndex, *StateEnum->GetDisplayNameTextByValue(ActorData.CurrentState).ToString(), ActorIndex);
				}
#endif 

				SetComponentUsage(true, ActorData.CurrentState, ActorData.PermutationIndex);
			}
			
			// Propagate visibility to leader component
			if (ActorData.bRequiresTick)
			{
				SetComponentTick(ActorData.CurrentState, ActorData.PermutationIndex);
			}
		}
	}
}

void UAnimSharingInstance::RemoveFromCurrentBlend(int32 ActorIndex)
{
	if (PerActorData[ActorIndex].bBlending && PerActorData[ActorIndex].BlendInstanceIndex != INDEX_NONE && BlendInstances.IsValidIndex(PerActorData[ActorIndex].BlendInstanceIndex))
	{
		FBlendInstance& OldBlendInstance = BlendInstances[PerActorData[ActorIndex].BlendInstanceIndex];
		SetLeaderComponentForActor(ActorIndex, OldBlendInstance.TransitionBlendInstance->GetToComponent());
		OldBlendInstance.ActorIndices.Remove(ActorIndex);
		PerActorData[ActorIndex].BlendInstanceIndex = INDEX_NONE;
	}
}

void UAnimSharingInstance::RemoveFromCurrentOnDemand(int32 ActorIndex)
{
	if (PerActorData[ActorIndex].bRunningOnDemand && PerActorData[ActorIndex].OnDemandInstanceIndex != INDEX_NONE && OnDemandInstances.IsValidIndex(PerActorData[ActorIndex].OnDemandInstanceIndex))
	{
		OnDemandInstances[PerActorData[ActorIndex].OnDemandInstanceIndex].ActorIndices.Remove(ActorIndex);
	}
}

void UAnimSharingInstance::TickBlendInstances()
{
	SCOPE_CYCLE_COUNTER(STAT_AnimationSharing_UpdateBlends);
	for (int32 InstanceIndex = 0; InstanceIndex < BlendInstances.Num(); ++InstanceIndex)
	{
		FBlendInstance& Instance = BlendInstances[InstanceIndex];
		checkf(Instance.bActive, TEXT("Blends should be active at this point"));

		/** Check whether or not the blend has ended */
		if (Instance.EndTime <= WorldTime)
		{
#if LOG_STATES
			UE_LOGF(LogAnimationSharing, Log, "Finished blend %ls from %ls [%i]", *StateEnum->GetDisplayNameTextByValue(Instance.StateTo).ToString(), *StateEnum->GetDisplayNameTextByValue(Instance.StateFrom).ToString(), InstanceIndex);
#endif

			// Finish blend into unique animation, need to just set it to use the correct component
			const bool bToStateIsOnDemand = PerStateData[Instance.StateTo].bIsOnDemand;
			const bool bFromStateIsOnDemand = PerStateData[Instance.StateFrom].bIsOnDemand;

			// If we were blending to an on-demand state we need to set the on-demand component as the new leader component
			if (bToStateIsOnDemand)
			{
				for (uint32 ActorIndex : Instance.ActorIndices)
				{
					SetLeaderComponentForActor(ActorIndex, Instance.TransitionBlendInstance->GetToComponent());
					PerActorData[ActorIndex].PermutationIndex = 0;
#if LOG_STATES
					UE_LOGF(LogAnimationSharing, Log, "Setting %i to on-demand component %i [%i]", ActorIndex, Instance.ToOnDemandInstanceIndex, InstanceIndex);
#endif

					for (uint32 ComponentIndex : PerActorData[ActorIndex].ComponentIndices)
					{
						if (USkeletalMeshComponent* Component = GetComponent(PerComponentData[ComponentIndex]))
						{
							UAnimationSharingManager::SetDebugMaterial(Component, 0);
						}
					}
				}
			}
			/** Otherwise if the state we were blending from was not on-demand we set the new state component as the new leader component,
				if we are blending from an on-demand state FOnDemandInstance with set the correct leader component when it finishes	*/
			else if (!bFromStateIsOnDemand)
			{				
				for (uint32 ActorIndex : Instance.ActorIndices)
				{
					if (PerActorData[ActorIndex].CurrentState == Instance.StateTo)
					{
#if LOG_STATES 
						UE_LOGF(LogAnimationSharing, Log, "Setting %i to state %i | %i [%i]", ActorIndex, Instance.StateTo, Instance.ToPermutationIndex, InstanceIndex);
#endif
						SetPermutationFollowerComponent(Instance.StateTo, ActorIndex, Instance.ToPermutationIndex);
#if !UE_BUILD_SHIPPING
						for (uint32 ComponentIndex : PerActorData[ActorIndex].ComponentIndices)
						{
							if (USkeletalMeshComponent* Component = GetComponent(PerComponentData[ComponentIndex]))
							{
								UAnimationSharingManager::SetDebugMaterial(Component, 1);
							}
						}
#endif
					}
				
				}
			}			

			// Free up the used blend actor
			FreeBlendInstance(Instance.TransitionBlendInstance);

			// Clear flags and index on the actor data as the blend has finished
			for (uint32 ActorIndex : Instance.ActorIndices)
			{
				PerActorData[ActorIndex].BlendInstanceIndex = INDEX_NONE;
				PerActorData[ActorIndex].bBlending = false;
			}

			// Remove this blend instance as it has finished
			RemoveBlendInstance(InstanceIndex);
			--InstanceIndex;
		}
		else
		{
			// Check whether or not the blend has started, if not set up the actors as followers at this point
			if (!Instance.bBlendStarted)
			{
				for (uint32 ActorIndex : Instance.ActorIndices)
				{
					SetLeaderComponentForActor(ActorIndex, Instance.TransitionBlendInstance->GetComponent());			

					for (uint32 ComponentIndex : PerActorData[ActorIndex].ComponentIndices)
					{
						if (USkeletalMeshComponent* Component = GetComponent(PerComponentData[ComponentIndex]))
						{
							UAnimationSharingManager::SetDebugMaterial(Component, 2);
						}
					}
				}

				Instance.bBlendStarted = true;
			}

			const bool bShouldTick = DoAnyActorsRequireTicking(Instance);

			if (!PerStateData[Instance.StateFrom].bIsOnDemand)
			{
				SetComponentUsage(true, Instance.StateFrom, Instance.FromPermutationIndex);
				if (bShouldTick)
				{
					SetComponentTick(Instance.StateFrom, Instance.FromPermutationIndex);
				}
			}

			if (!PerStateData[Instance.StateTo].bIsOnDemand)
			{
				SetComponentUsage(true, Instance.StateTo, Instance.ToPermutationIndex);
				if (bShouldTick)
				{
					SetComponentTick(Instance.StateTo, Instance.ToPermutationIndex);
				}
			}
		}
	}
}

void UAnimSharingInstance::TickAnimationStates()
{
	for (FPerStateData& StateData : PerStateData)
	{
		for (int32 Index = 0; Index < StateData.Components.Num(); ++Index)
		{
			const bool bPreviousState = StateData.PreviousInUseComponentFrameBits[Index];
			const bool bCurrentState = StateData.InUseComponentFrameBits[Index];

			const bool bShouldTick = StateData.FollowerTickRequiredFrameBits[Index];

			if (bCurrentState != bPreviousState)
			{
				if (bCurrentState)
				{
					// Turn on
					UAnimationSharingManager::SetDebugMaterial(StateData.Components[Index], 1);
					StateData.Components[Index]->SetComponentTickEnabled(true);
				}
				else
				{
					// Turn off
					UAnimationSharingManager::SetDebugMaterial(StateData.Components[Index], 0);
					StateData.Components[Index]->SetComponentTickEnabled(false);
				}
			}
			else if (!bCurrentState && StateData.Components[Index]->IsComponentTickEnabled())
			{
				// Turn off
				UAnimationSharingManager::SetDebugMaterial(StateData.Components[Index], 0);
				StateData.Components[Index]->SetComponentTickEnabled(false);
			}

			StateData.Components[Index]->bRecentlyRendered = bShouldTick;
			StateData.Components[Index]->VisibilityBasedAnimTickOption = bShouldTick ? EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones : EVisibilityBasedAnimTickOption::OnlyTickPoseWhenRendered;
		}

		// Set previous to current and reset current bits
		StateData.PreviousInUseComponentFrameBits = StateData.InUseComponentFrameBits;
		StateData.InUseComponentFrameBits.Init(false, StateData.PreviousInUseComponentFrameBits.Num());
		StateData.FollowerTickRequiredFrameBits.Init(false, StateData.FollowerTickRequiredFrameBits.Num());

		/** Reset on demand index for next frame */
		StateData.CurrentFrameOnDemandIndex = INDEX_NONE;
	}	
}

void UAnimSharingInstance::SetComponentUsage(bool bUsage, uint8 StateIndex, uint32 ComponentIndex)
{
	// TODO component index should always be valid
#if LOG_STATES 
	if (!PerStateData[StateIndex].InUseComponentFrameBits.IsValidIndex(ComponentIndex))
	{
		UE_LOGF(LogAnimationSharing, Log, "Invalid set component usage %i is out of range of %i for state %ls by component %i", ComponentIndex, PerStateData[StateIndex].Components.Num(), *StateEnum->GetDisplayNameTextByValue(StateIndex).ToString(), ComponentIndex);
	}
#endif

	if (PerStateData.IsValidIndex(StateIndex) && PerStateData[StateIndex].InUseComponentFrameBits.IsValidIndex(ComponentIndex))
	{
		PerStateData[StateIndex].InUseComponentFrameBits[ComponentIndex] = bUsage;
	}
}

void UAnimSharingInstance::SetComponentTick(uint8 StateIndex, uint32 ComponentIndex)
{
	if (PerStateData[StateIndex].FollowerTickRequiredFrameBits.IsValidIndex(ComponentIndex))
	{
		PerStateData[StateIndex].FollowerTickRequiredFrameBits[ComponentIndex] = true;
	}
}

void UAnimSharingInstance::FreeBlendInstance(FTransitionBlendInstance* Instance)
{
	Instance->Stop();
	BlendInstanceStack.FreeInstance(Instance);
}

void UAnimSharingInstance::FreeAdditiveInstance(uint8 StateIndex, FAdditiveAnimationInstance* Instance)
{
	Instance->Stop();

	PerStateData[StateIndex].AdditiveInstanceStack.FreeInstance(Instance);
}

void UAnimSharingInstance::SetLeaderComponentForActor(uint32 ActorIndex, USkeletalMeshComponent* Component)
{
	// Always ensure the component is ticking
	if (Component)
	{
		Component->SetComponentTickEnabled(true);
	}

	const FPerActorData& ActorData = PerActorData[ActorIndex];
	// Do not update the component of the additive actor itself, otherwise update the base component
	if (ActorData.bRunningAdditive && AdditiveInstances.IsValidIndex(ActorData.AdditiveInstanceIndex) && AdditiveInstances[ActorData.AdditiveInstanceIndex].AdditiveAnimationInstance->GetComponent() != Component)
	{
#if LOG_STATES
		UE_LOGF(LogAnimationSharing, Log, "Updating additive instance base component %ls for %i", *StateEnum->GetDisplayNameTextByValue(AdditiveInstances[ActorData.AdditiveInstanceIndex].State).ToString(), ActorIndex);
#endif // LOG_STATES

		AdditiveInstances[ActorData.AdditiveInstanceIndex].BaseComponent = Component;
		AdditiveInstances[ActorData.AdditiveInstanceIndex].AdditiveAnimationInstance->UpdateBaseComponent(Component);

		return;
	}
	
	for (uint32 ComponentIndex : ActorData.ComponentIndices)
	{
		if (USkeletalMeshComponent* ActorComponent = GetComponent(PerComponentData[ComponentIndex]))
		{
			ActorComponent->SetLeaderPoseComponent(Component, true);
		}
	}
}

void UAnimSharingInstance::SetupFollowerComponent(uint8 CurrentState, uint32 ActorIndex)
{
	const FPerStateData& StateData = PerStateData[CurrentState];

	if (StateData.Components.Num() == 0)
	{	
		UE_LOGF(LogAnimationSharing, Warning, "No Leader Components available for state %ls, make sure to set up an Animation Sequence/Blueprint ", *StateEnum->GetDisplayNameTextByValue(CurrentState).ToString());
		return;
	}

	if (!StateData.bIsOnDemand)
	{
		const uint32 PermutationIndex = DeterminePermutationIndex(ActorIndex, CurrentState);
		SetPermutationFollowerComponent(CurrentState, ActorIndex, PermutationIndex);
	}
	else
	{
		const uint32 OnDemandInstanceIndex = SetupOnDemandInstance(CurrentState);

		if (OnDemandInstanceIndex != INDEX_NONE)
		{
			USkeletalMeshComponent* LeaderComponent = StateData.Components[OnDemandInstances[OnDemandInstanceIndex].UsedPerStateComponentIndex];
			SetLeaderComponentForActor(ActorIndex, LeaderComponent);
			OnDemandInstances[OnDemandInstanceIndex].ActorIndices.Add(ActorIndex);

			PerActorData[ActorIndex].OnDemandInstanceIndex = OnDemandInstanceIndex;
			PerActorData[ActorIndex].bRunningOnDemand = true;

			// TODO do we need to reset
			PerActorData[ActorIndex].PermutationIndex = 0;				
		}
	}
}

void UAnimSharingInstance::SetPermutationFollowerComponent(uint8 StateIndex, uint32 ActorIndex, uint32 PermutationIndex)
{
	const FPerStateData& StateData = PerStateData[StateIndex];

	// TODO Min should not be needed if PermutationIndex is always valid
	PermutationIndex = FMath::Min((uint32)StateData.Components.Num() - 1, PermutationIndex);
#if LOG_STATES 
	if (!StateData.Components.IsValidIndex(PermutationIndex))
	{
		UE_LOGF(LogAnimationSharing, Log, "Invalid set component usage %i is out of range of %i for state %ls by actor %i", PermutationIndex, StateData.Components.Num(), *StateEnum->GetDisplayNameTextByValue(StateIndex).ToString(), ActorIndex);
	}
#endif

	SetLeaderComponentForActor(ActorIndex, StateData.Components[PermutationIndex]);
	PerActorData[ActorIndex].PermutationIndex = IntCastChecked<uint8>(PermutationIndex);
	UAnimationSharingManager::SetDebugMaterial(StateData.Components[PermutationIndex], 1);
}

uint32 UAnimSharingInstance::DeterminePermutationIndex(uint32 ActorIndex, uint8 State) const
{
	const FPerStateData& StateData = PerStateData[State];
	const TArray<USkeletalMeshComponent*>& Components = StateData.Components;

	// This can grow to be more intricate to take into account surrounding actors?
	const uint32 PermutationIndex = FMath::RandHelper(Components.Num());
	checkf(Components.IsValidIndex(PermutationIndex), TEXT("Not enough LeaderComponents initialised!"));

	return PermutationIndex;
}

uint32 UAnimSharingInstance::SetupBlend(uint8 FromState, uint8 ToState, uint32 ActorIndex)
{
	const bool bConcurrentBlendsReached = !BlendInstanceStack.InstanceAvailable();
	const bool bOnDemand = PerStateData[ToState].bIsOnDemand;

	uint32 BlendInstanceIndex = INDEX_NONE;
	if (!bConcurrentBlendsReached)
	{
		BlendInstanceIndex = BlendInstances.IndexOfByPredicate([this, FromState, ToState, bOnDemand, ActorIndex](const FBlendInstance& Instance)
		{			
			return (!Instance.bActive &&				// The instance should not have started yet
				Instance.StateFrom == FromState &&		// It should be blending from the same state
				Instance.StateTo == ToState &&			// It should be blending to the same state
				Instance.bOnDemand == bOnDemand &&		// It should match whether or not it is an on-demand state QQQ is this needed?
				Instance.FromPermutationIndex == PerActorData[ActorIndex].PermutationIndex); // It should be blending from the same permutation inside of the state 
		});

		FBlendInstance* BlendInstance = BlendInstanceIndex != INDEX_NONE ? &BlendInstances[BlendInstanceIndex] : nullptr;

		if (!BlendInstance)
		{
			BlendInstance = &BlendInstances.AddDefaulted_GetRef();
			BlendInstanceIndex = BlendInstances.Num() - 1;
			BlendInstance->bActive = false;
			BlendInstance->FromOnDemandInstanceIndex = BlendInstance->ToOnDemandInstanceIndex = INDEX_NONE;
			BlendInstance->StateFrom = FromState;
			BlendInstance->StateTo = ToState;
			BlendInstance->BlendTime = CalculateBlendTime( ToState);
			BlendInstance->bOnDemand = bOnDemand;
			BlendInstance->EndTime = static_cast<float>(GetWorld()->GetTimeSeconds()) + BlendInstance->BlendTime;
			BlendInstance->TransitionBlendInstance = BlendInstanceStack.GetInstance();

			BlendInstance->TransitionBlendInstance->GetComponent()->SetComponentTickEnabled(true);

			// Setup permutation indices to and from we are blending
			BlendInstance->FromPermutationIndex = PerActorData[ActorIndex].PermutationIndex;
			BlendInstance->ToPermutationIndex = DeterminePermutationIndex( ActorIndex, ToState);
		}

		checkf(BlendInstance, TEXT("Unable to create blendcontainer"));

		BlendInstance->ActorIndices.Add(ActorIndex);
		PerActorData[ActorIndex].bBlending = true;
	}

	return BlendInstanceIndex;
}

uint32 UAnimSharingInstance::SetupBlendFromOnDemand(uint8 ToState, uint32 OnDemandInstanceIndex, uint32 ActorIndex)
{
	const uint8 FromState = OnDemandInstances[OnDemandInstanceIndex].State;
	const uint32 BlendInstanceIndex = SetupBlend(FromState, ToState, ActorIndex);

	if (BlendInstanceIndex != INDEX_NONE)
	{
		BlendInstances[BlendInstanceIndex].FromOnDemandInstanceIndex = OnDemandInstanceIndex;
	}

	return BlendInstanceIndex;
}

uint32 UAnimSharingInstance::SetupBlendBetweenOnDemands(uint8 FromOnDemandInstanceIndex, uint32 ToOnDemandInstanceIndex, uint32 ActorIndex)
{
	const uint8 FromState = OnDemandInstances[FromOnDemandInstanceIndex].State;
	const uint8 ToState = OnDemandInstances[ToOnDemandInstanceIndex].State;
	const uint32 BlendInstanceIndex = SetupBlend(FromState, ToState, ActorIndex);

	if (BlendInstanceIndex != INDEX_NONE)
	{
		BlendInstances[BlendInstanceIndex].FromOnDemandInstanceIndex = FromOnDemandInstanceIndex;
		BlendInstances[BlendInstanceIndex].ToOnDemandInstanceIndex = ToOnDemandInstanceIndex;
	}

	return BlendInstanceIndex;
}

uint32 UAnimSharingInstance::SetupBlendToOnDemand(uint8 FromState, uint32 ToOnDemandInstanceIndex, uint32 ActorIndex)
{
	const uint8 ToState = OnDemandInstances[ToOnDemandInstanceIndex].State;
	const uint32 BlendInstanceIndex = SetupBlend(FromState, ToState, ActorIndex);

	if (BlendInstanceIndex != INDEX_NONE)
	{
		BlendInstances[BlendInstanceIndex].ToOnDemandInstanceIndex = ToOnDemandInstanceIndex;
	}

	return BlendInstanceIndex;
}

void UAnimSharingInstance::SwitchBetweenOnDemands(uint32 FromOnDemandInstanceIndex, uint32 ToOnDemandInstanceIndex, uint32 ActorIndex)
{
	/** Remove this actor from the currently running on-demand instance */
	if (FromOnDemandInstanceIndex != INDEX_NONE)
	{
		OnDemandInstances[FromOnDemandInstanceIndex].ActorIndices.Remove(ActorIndex);
	}

	const FOnDemandInstance& Instance = OnDemandInstances[ToOnDemandInstanceIndex];
	const uint32 ComponentIndex = Instance.UsedPerStateComponentIndex;
	const uint32 StateIndex = Instance.State;
	PerActorData[ActorIndex].PermutationIndex = 0;
	SetLeaderComponentForActor(ActorIndex, PerStateData[StateIndex].Components[ComponentIndex]);
}

uint32 UAnimSharingInstance::SetupOnDemandInstance(uint8 StateIndex)
{
	uint32 InstanceIndex = INDEX_NONE;

	FPerStateData& StateData = PerStateData[StateIndex];
	if (StateData.CurrentFrameOnDemandIndex != INDEX_NONE && OnDemandInstances.IsValidIndex(StateData.CurrentFrameOnDemandIndex))
	{
		InstanceIndex = StateData.CurrentFrameOnDemandIndex;
	}
	else
	{
		// Otherwise we'll need to kick one of right now so try and set one up		
		if (StateData.Components.Num())
		{
			auto FindRandomAvailableIndex = [&StateData]() -> uint32
				{
					TBitArray<>& ComponentFrameBits = StateData.InUseComponentFrameBits;
					TArray<uint32> ZeroBitIndices;
					ZeroBitIndices.Reserve(ComponentFrameBits.Num());

					for (int32 Index = 0; Index < ComponentFrameBits.Num(); ++Index)
					{
						if (ComponentFrameBits[Index] == 0)
						{
							ZeroBitIndices.Add(Index);
						}
					}

					if (ZeroBitIndices.Num() > 0)
					{
						const uint32 RandomIndex = FMath::RandHelper(ZeroBitIndices.Num());
						ComponentFrameBits[ZeroBitIndices[RandomIndex]] = 1;
						return ZeroBitIndices[RandomIndex];
					}

					// No available components this frame.
					return INDEX_NONE;
				};

			const uint32 AvailableIndex = StateData.bShouldRandomizeOnDemandAnims && StateData.InUseComponentFrameBits.Num() > 1 ? FindRandomAvailableIndex() : StateData.InUseComponentFrameBits.FindAndSetFirstZeroBit();
			
			if (AvailableIndex != INDEX_NONE)
			{
				FOnDemandInstance& Instance = OnDemandInstances.AddDefaulted_GetRef();
				InstanceIndex = OnDemandInstances.Num() - 1;
				StateData.CurrentFrameOnDemandIndex = InstanceIndex;

				Instance.bActive = 0;
				Instance.bBlendActive = 0;
				Instance.State = StateIndex;
				Instance.ForwardState = StateData.bShouldForwardToState ? StateData.ForwardStateValue : INDEX_NONE;
				Instance.UsedPerStateComponentIndex = AvailableIndex;
				Instance.bReturnToPreviousState = StateData.bReturnToPreviousState;
				Instance.StartTime = 0.f;
				Instance.BlendToPermutationIndex = INDEX_NONE;

				const float WorldTimeSeconds = static_cast<float>(GetWorld()->GetTimeSeconds());
				Instance.EndTime = WorldTimeSeconds + StateData.AnimationLengths[AvailableIndex];
				Instance.StartBlendTime = Instance.EndTime - CalculateBlendTime(StateIndex);

				USkeletalMeshComponent* FreeComponent = StateData.Components[AvailableIndex];

				UAnimationSharingManager::SetDebugMaterial(FreeComponent, 1);

				FreeComponent->SetComponentTickEnabled(true);
				FreeComponent->SetPosition(0.f, false);
				FreeComponent->Play(false);
#if LOG_STATES 
				UE_LOGF(LogAnimationSharing, Log, "Setup on demand state %ls %i", *StateEnum->GetDisplayNameTextByValue(StateIndex).ToString(), InstanceIndex);
#endif
			}
			else
			{
				// Next resort
				const float MaxStartTime = WorldTime - PerStateData[StateIndex].WiggleTime;
				float WiggleStartTime = TNumericLimits<float>::Max();
				float NonWiggleStartTime = TNumericLimits<float>::Max();
				int32 WiggleIndex = INDEX_NONE;
				int32 NonWiggleIndex = INDEX_NONE;
				for (int32 RunningInstanceIndex = 0; RunningInstanceIndex < OnDemandInstances.Num(); ++RunningInstanceIndex)
				{
					FOnDemandInstance& Instance = OnDemandInstances[RunningInstanceIndex];

					if (Instance.State == StateIndex)
					{
						if (Instance.StartTime <= MaxStartTime && Instance.StartTime < WiggleStartTime)
						{
							WiggleStartTime = Instance.StartTime;
							WiggleIndex = RunningInstanceIndex;
						}
						else if (Instance.StartTime < NonWiggleStartTime)
						{
							NonWiggleStartTime = Instance.StartTime;
							NonWiggleIndex = RunningInstanceIndex;							
						}
					}
				}

				// Snap to on demand instance that has started last within the number of wiggle frames
				if (WiggleIndex != INDEX_NONE)
				{
					InstanceIndex = WiggleIndex;
				}
				// Snap to closest on demand instance outside of the number of wiggle frames
				else if (NonWiggleIndex != INDEX_NONE)
				{
					InstanceIndex = NonWiggleIndex;
				}
				else
				{
					// No instances available and none actually currently running this state, should probably up the number of available concurrent on demand instances at this point
					UE_LOGF(LogAnimationSharing, Warning, "No more on demand components available");
				}
			}
		}
	}

	return InstanceIndex;
}

uint32 UAnimSharingInstance::SetupAdditiveInstance(uint8 StateIndex, USkeletalMeshComponent* InBaseComponent)
{
	uint32 InstanceIndex = INDEX_NONE;

	FPerStateData& StateData = PerStateData[StateIndex];
	if (StateData.AdditiveInstanceStack.InstanceAvailable())
	{
		FAdditiveAnimationInstance* AnimationInstance = StateData.AdditiveInstanceStack.GetInstance();
		FAdditiveInstance& Instance = AdditiveInstances.AddDefaulted_GetRef();
		Instance.bActive = false;
		Instance.AdditiveAnimationInstance = AnimationInstance;
		Instance.BaseComponent = InBaseComponent;
		const float WorldTimeSeconds = static_cast<float>(GetWorld()->GetTimeSeconds());
		Instance.EndTime = WorldTimeSeconds + StateData.AdditiveAnimationSequence->GetPlayLength();
		Instance.State = StateIndex;
		Instance.UsedPerStateComponentIndex = PerStateData[StateIndex].Components.IndexOfByKey(AnimationInstance->GetComponent());

		InstanceIndex = AdditiveInstances.Num() - 1;
		AnimationInstance->Setup(Instance.BaseComponent, StateData.AdditiveAnimationSequence);

		SetComponentUsage(true, Instance.State, Instance.UsedPerStateComponentIndex);
		SetComponentTick(Instance.State, Instance.UsedPerStateComponentIndex);
	}

	return InstanceIndex;
}

void UAnimSharingInstance::KickoffInstances()
{
	SCOPE_CYCLE_COUNTER(STAT_AnimationSharing_KickoffInstances);
	for (FBlendInstance& BlendInstance : BlendInstances)
	{
		if (!BlendInstance.bActive)
		{
			BlendInstance.bBlendStarted = false;

			FString ActorIndicesString;
			for (uint32 ActorIndex : BlendInstance.ActorIndices)
			{
				if (ActorIndex != BlendInstance.ActorIndices.Last())
				{
					ActorIndicesString += FString::Printf(TEXT("%i, "), ActorIndex);
				}
				else
				{
					ActorIndicesString += FString::Printf(TEXT("%i"), ActorIndex);
				}
			}
#if LOG_STATES 
			UE_LOGF(LogAnimationSharing, Log, "Starting blend from %ls to %ls [%ls]", *StateEnum->GetDisplayNameTextByValue(BlendInstance.StateFrom).ToString(), *StateEnum->GetDisplayNameTextByValue(BlendInstance.StateTo).ToString(), *ActorIndicesString);
#endif

			// TODO should be able to assume permutation indices are valid here
			BlendInstance.FromPermutationIndex = FMath::Min((uint32)PerStateData[BlendInstance.StateFrom].Components.Num() - 1, BlendInstance.FromPermutationIndex);
			BlendInstance.ToPermutationIndex = FMath::Min((uint32)PerStateData[BlendInstance.StateTo].Components.Num() - 1, BlendInstance.ToPermutationIndex);

			USkeletalMeshComponent* From = PerStateData[BlendInstance.StateFrom].Components[BlendInstance.FromPermutationIndex];
			USkeletalMeshComponent* To = PerStateData[BlendInstance.StateTo].Components[BlendInstance.ToPermutationIndex];

			if (PerStateData[BlendInstance.StateTo].bIsOnDemand && (BlendInstance.ToOnDemandInstanceIndex != INDEX_NONE))
			{
				To = PerStateData[BlendInstance.StateTo].Components[OnDemandInstances[BlendInstance.ToOnDemandInstanceIndex].UsedPerStateComponentIndex];		
			}

			if (PerStateData[BlendInstance.StateFrom].bIsOnDemand && (BlendInstance.FromOnDemandInstanceIndex != INDEX_NONE))
			{
				const uint32 UsedComponentIndex = OnDemandInstances[BlendInstance.FromOnDemandInstanceIndex].UsedPerStateComponentIndex;
				From = PerStateData[BlendInstance.StateFrom].Components[UsedComponentIndex];
			}

			for (uint32 ActorIndex : BlendInstance.ActorIndices)
			{
				PerActorData[ActorIndex].PermutationIndex = IntCastChecked<uint8>(BlendInstance.ToPermutationIndex);
				PerActorData[ActorIndex].bBlending = true;
			}

			BlendInstance.TransitionBlendInstance->Setup(From, To, BlendInstance.BlendTime);
			BlendInstance.bActive = true;
		}
	}

	for (FOnDemandInstance& OnDemandInstance : OnDemandInstances)
	{
		if (!OnDemandInstance.bActive)
		{
			OnDemandInstance.bActive = true;
			OnDemandInstance.StartTime = WorldTime;
		}
	}
}

float UAnimSharingInstance::CalculateBlendTime(uint8 StateIndex) const
{
	checkf(PerStateData.IsValidIndex(StateIndex), TEXT("Invalid State index"));
	return PerStateData[StateIndex].BlendTime;
}

void UAnimSharingInstance::RemoveComponent(int32 ComponentIndex)
{
	if (PerComponentData.Num() > 1 && ComponentIndex != PerComponentData.Num() - 1)
	{
		// Index of the component we will swap with
		const uint32 SwapIndex = PerComponentData.Num() - 1;

		// Find actor for component we will swap with
		const uint32 SwapActorIndex = PerComponentData[SwapIndex].ActorIndex;

		// Update component index in the actor to match with ComponentIndex (which it will be swapped with)
		const uint32 ActorDataComponentIndex = PerActorData[SwapActorIndex].ComponentIndices.IndexOfByKey(SwapIndex);
		if (ActorDataComponentIndex != INDEX_NONE)
		{
			PerActorData[SwapActorIndex].ComponentIndices[ActorDataComponentIndex] = ComponentIndex;
		}
	}

	PerComponentData.RemoveAtSwap(ComponentIndex, EAllowShrinking::No);
}

void UAnimSharingInstance::RemoveBlendInstance(int32 InstanceIndex)
{
	FBlendInstance& Instance = BlendInstances[InstanceIndex];

	// Index we could swap with
	const uint32 SwapIndex = BlendInstances.Num() - 1;
	if (BlendInstances.Num() > 1 && InstanceIndex != SwapIndex)
	{
		FBlendInstance& SwapInstance = BlendInstances[SwapIndex];
		// Remap all of the actors to point to our new index
		for (uint32 ActorIndex : SwapInstance.ActorIndices)
		{
			PerActorData[ActorIndex].BlendInstanceIndex = InstanceIndex;
		}
	}

	BlendInstances.RemoveAtSwap(InstanceIndex, EAllowShrinking::No);
}

void UAnimSharingInstance::RemoveOnDemandInstance(int32 InstanceIndex)
{
	const FOnDemandInstance& Instance = OnDemandInstances[InstanceIndex];

	// Index we could swap with
	const uint32 SwapIndex = OnDemandInstances.Num() - 1;
	if (OnDemandInstances.Num() > 1 && InstanceIndex != SwapIndex)
	{
		const FOnDemandInstance& SwapInstance = OnDemandInstances[SwapIndex];
		// Remap all of the actors to point to our new index
		for (uint32 ActorIndex : SwapInstance.ActorIndices)
		{
			// Only remap if it's still part of this instance
			const bool bPartOfOtherOnDemand = PerActorData[ActorIndex].OnDemandInstanceIndex != InstanceIndex;
			// Could be swapping with other instance in which case we should update the index
			const bool bShouldUpdateIndex = !bPartOfOtherOnDemand || (PerActorData[ActorIndex].OnDemandInstanceIndex == SwapIndex);
			
			if (bShouldUpdateIndex)
			{
				PerActorData[ActorIndex].OnDemandInstanceIndex = InstanceIndex;
			}		
		}
	}

	// Remove and swap 
	OnDemandInstances.RemoveAtSwap(InstanceIndex, EAllowShrinking::No);
}

void UAnimSharingInstance::RemoveAdditiveInstance(int32 InstanceIndex)
{
	FAdditiveInstance& Instance = AdditiveInstances[InstanceIndex];

	// Index we could swap with
	const uint32 SwapIndex = AdditiveInstances.Num() - 1;
	if (AdditiveInstances.Num() > 1 && InstanceIndex != SwapIndex)
	{
		FAdditiveInstance& SwapInstance = AdditiveInstances[SwapIndex];
		// Remap all of the actors to point to our new index
		if (SwapInstance.ActorIndex != INDEX_NONE)
		{
			PerActorData[SwapInstance.ActorIndex].AdditiveInstanceIndex = InstanceIndex;
		}
	}

	AdditiveInstances.RemoveAtSwap(InstanceIndex, EAllowShrinking::No);
}

bool UAnimationSharingManager::CheckDataForActor(AActor* InActor) const
{
	int32 PerSkeletonDataIdx = INDEX_NONE;
	int32 RegisteredActorsIdx = INDEX_NONE;

	for (int32 SkeletonIndex = 0; SkeletonIndex < PerSkeletonData.Num(); ++SkeletonIndex)
	{
		const UAnimSharingInstance* SkeletonData = PerSkeletonData[SkeletonIndex];
		const int32 ActorIndex = SkeletonData->RegisteredActors.IndexOfByKey(InActor);

		if (ActorIndex != INDEX_NONE)
		{
			PerSkeletonDataIdx = SkeletonIndex;
			RegisteredActorsIdx = ActorIndex;
			break;
		}
	}

	if (PerSkeletonDataIdx == INDEX_NONE)
	{
		UE_LOGF(LogAnimationSharing, Warning, "CheckActorData Failed! Reason: Actor %ls is not registered. PerSkeletonData Num: %d",
			*GetNameSafe(InActor), PerSkeletonData.Num());
		return false;
	}

	const UAnimSharingInstance* SkeletonData = PerSkeletonData[PerSkeletonDataIdx];
	if (!SkeletonData->PerActorData.IsValidIndex(RegisteredActorsIdx))
	{
		UE_LOGF(LogAnimationSharing, Warning, "CheckActorData Failed! Reason: ActorIndex '%d' for '%ls' is not a valid index in PerActorData. SkeletonDataIdx: %d",
			RegisteredActorsIdx, *GetNameSafe(InActor), PerSkeletonDataIdx);
		return false;
	}

	const FPerActorData& ActorData = SkeletonData->PerActorData[RegisteredActorsIdx];
	if (ActorData.ComponentIndices.Num() == 0)
	{
		UE_LOGF(LogAnimationSharing, Warning, "CheckActorData Failed! Reason: ComponentIndices for '%ls' is empty. SkeletonDataIdx: %d ActorDataIdx: %d",
			*GetNameSafe(InActor), PerSkeletonDataIdx, RegisteredActorsIdx);
		return false;
	}

	for (int32 ComponentIndex : ActorData.ComponentIndices)
	{
		if (!SkeletonData->PerComponentData.IsValidIndex(ComponentIndex))
		{
			UE_LOGF(LogAnimationSharing, Warning, "CheckActorData Failed! Reason: ComponentIndex '%d' for '%ls' is not a valid index in PerComponentData. PerComponentData.Num: %d",
				ComponentIndex, *GetNameSafe(InActor), SkeletonData->PerComponentData.Num());

			return false;
		}
	}

	return true;
}

void UAnimationSharingManager::LogData() const
{
	UE_LOGF(LogAnimationSharing, Log, "PerSkeletonData Num: %d", PerSkeletonData.Num());
	for (int32 SkeletonIndex = 0; SkeletonIndex < PerSkeletonData.Num(); ++SkeletonIndex)
	{
		if (const UAnimSharingInstance* SkeletonData = PerSkeletonData[SkeletonIndex])
		{
			UE_LOGF(LogAnimationSharing, Log, "\t RegisteredActors Num: %d", SkeletonData->RegisteredActors.Num());
			for (int32 Idx = 0; Idx < SkeletonData->RegisteredActors.Num(); Idx++)
			{
				UE_LOGF(LogAnimationSharing, Log, "\t\t Idx: %d Actor: %ls", Idx, *GetNameSafe(SkeletonData->RegisteredActors[Idx]));
			}

			UE_LOGF(LogAnimationSharing, Log, "\t PerActorData Num: %d", SkeletonData->PerActorData.Num());
			for (int32 Idx = 0; Idx < SkeletonData->PerActorData.Num(); Idx++)
			{
				const FPerActorData& ActorData = SkeletonData->PerActorData[Idx];
				const FString ComponentIndicesStr = FString::JoinBy(ActorData.ComponentIndices, TEXT(", "), [](uint32 Item) { return FString::FromInt(Item); });

				UE_LOGF(LogAnimationSharing, Log, "\t\t Idx: %d CurrState: %d PrevState: %d PermutationIdx: %d ComponentIndices: %ls",
					Idx, ActorData.CurrentState, ActorData.PreviousState, ActorData.PermutationIndex, *ComponentIndicesStr);
			}

			UE_LOGF(LogAnimationSharing, Log, "\t PerComponentData Num: %d", SkeletonData->PerComponentData.Num());
			for (int32 Idx = 0; Idx < SkeletonData->PerComponentData.Num(); Idx++)
			{
				const FPerComponentData& PerComponentData = SkeletonData->PerComponentData[Idx];
				const USkeletalMeshComponent* SkelMeshComp = PerComponentData.WeakComponent.Get();

				UE_LOGF(LogAnimationSharing, Log, "\t\t Idx: %d Comp: %ls CompOwner: %ls ActorIndex: %d",
					Idx, *GetNameSafe(SkelMeshComp), *GetNameSafe(SkelMeshComp ? SkelMeshComp->GetOwner() : nullptr), PerComponentData.ActorIndex);
			}
		}
	}
}
