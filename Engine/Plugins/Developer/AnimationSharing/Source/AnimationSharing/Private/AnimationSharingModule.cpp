// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationSharingModule.h"
#include "AnimationSharingManager.h"

IMPLEMENT_MODULE( FAnimSharingModule, AnimationSharing);

TMap<const UWorld*, TObjectPtr<UAnimationSharingManager>> FAnimSharingModule::WorldAnimSharingManagers;

FOnAnimationSharingManagerCreated FAnimSharingModule::OnAnimationSharingManagerCreated;
FOnAnimationSharingManagerSetupAdded FAnimSharingModule::OnAnimationSharingManagerSetupAdded;

void FAnimSharingModule::StartupModule()
{
	FWorldDelegates::OnPostWorldCleanup.AddStatic(&FAnimSharingModule::OnWorldCleanup);
}

void FAnimSharingModule::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& WorldAnimSharingManagerPair : WorldAnimSharingManagers)
	{
		Collector.AddReferencedObject(WorldAnimSharingManagerPair.Value, WorldAnimSharingManagerPair.Key);
	}

#if DEBUG_MATERIALS 
	for (auto& Material : UAnimationSharingManager::DebugMaterials)
	{
		Collector.AddReferencedObject(Material);
	}
#endif
}

bool FAnimSharingModule::CreateAnimationSharingManager(UWorld* InWorld, const UAnimationSharingSetup* Setup)
{
	return CreateAnimationSharingManager(InWorld, [Setup] (UWorld* InWorld)
		{
			UAnimationSharingManager* Manager = NewObject<UAnimationSharingManager>(InWorld);
			Manager->Initialise(Setup);
			return Manager;
		} 
	);
}

bool FAnimSharingModule::CreateAnimationSharingManager(UWorld* InWorld, FCreateAnimationSharingManagerFunc CreateAnimationSharingManagerFunc)
{
	if (InWorld && InWorld->IsGameWorld() && CreateAnimationSharingManagerFunc && UAnimationSharingManager::AnimationSharingEnabled() && !WorldAnimSharingManagers.Contains(InWorld))
	{
		UE_SCOPED_ENGINE_ACTIVITY(TEXT("Initializing Animation Sharing"));
		UAnimationSharingManager* Manager = CreateAnimationSharingManagerFunc(InWorld);
		WorldAnimSharingManagers.Add(InWorld, Manager);	
		OnAnimationSharingManagerCreated.Broadcast(Manager, InWorld);

		return true;
	}

	return false;
}

void FAnimSharingModule::OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{	
	WorldAnimSharingManagers.Remove(World);
}
