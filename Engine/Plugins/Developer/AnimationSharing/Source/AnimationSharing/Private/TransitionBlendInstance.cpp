// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransitionBlendInstance.h"
#include "AnimationSharingInstances.h"
#include "AnimationSharingManager.h"

FTransitionBlendInstance::FTransitionBlendInstance() : SkeletalMeshComponent(nullptr), TransitionInstance(nullptr), FromComponent(nullptr), ToComponent(nullptr), BlendTime(0.f), bBlendState(false) {}

void FTransitionBlendInstance::Initialise(USkeletalMeshComponent* InSkeletalMeshComponent, UClass* InAnimationBPClass)
{
	if (InSkeletalMeshComponent)
	{
		SkeletalMeshComponent = InSkeletalMeshComponent;
		if (InAnimationBPClass)
		{
			SkeletalMeshComponent->SetAnimInstanceClass(InAnimationBPClass);
			TransitionInstance = Cast<UAnimSharingTransitionInstance>(SkeletalMeshComponent->GetAnimInstance());
		}
		
		SkeletalMeshComponent->SetComponentTickEnabled(false);
		// Force component to always run at LOD0, to ensure all follower-components will always be at the same or higher LOD level for bone transforms
		SkeletalMeshComponent->SetForcedLOD(1);
		SkeletalMeshComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	}
}

void FTransitionBlendInstance::Setup(USkeletalMeshComponent* InFromComponent, USkeletalMeshComponent* InToComponent, float InBlendTime)
{
	UAnimationSharingManager::SetDebugMaterial(SkeletalMeshComponent, 1);
	SkeletalMeshComponent->SetComponentTickEnabled(true);
	BlendTime = InBlendTime;
	if (TransitionInstance)
	{
		if (TransitionInstance->bBlendBool)
		{
			TransitionInstance->FromComponent = FromComponent = InFromComponent;
			TransitionInstance->ToComponent = ToComponent = InToComponent;			
		}
		else
		{
			TransitionInstance->FromComponent = FromComponent = InToComponent;
			TransitionInstance->ToComponent = ToComponent = InFromComponent;
		}

		TransitionInstance->BlendTime = InBlendTime;
		TransitionInstance->bBlendBool = bBlendState = !TransitionInstance->bBlendBool;

		SkeletalMeshComponent->AddTickPrerequisiteComponent(FromComponent);
		SkeletalMeshComponent->AddTickPrerequisiteComponent(ToComponent);		
	}	
}

void FTransitionBlendInstance::Stop()
{
	if (TransitionInstance)
	{
		UAnimationSharingManager::SetDebugMaterial(SkeletalMeshComponent, 0);
		SkeletalMeshComponent->SetComponentTickEnabled(false);
		SkeletalMeshComponent->RemoveTickPrerequisiteComponent(FromComponent);
		SkeletalMeshComponent->RemoveTickPrerequisiteComponent(ToComponent);
	}
}

USkeletalMeshComponent* FTransitionBlendInstance::GetComponent() const
{
	return SkeletalMeshComponent;
}

USkeletalMeshComponent* FTransitionBlendInstance::GetToComponent() const
{
	return bBlendState == false ? ToComponent : FromComponent;
}

USkeletalMeshComponent* FTransitionBlendInstance::GetFromComponent() const
{
	return bBlendState == false ? FromComponent : ToComponent;
}
