// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

#define UE_API ANIMATIONSHARING_API

class UClass;
class USkeletalMeshComponent;

class UAnimSharingAdditiveInstance;
class UAnimSequence;

struct FAdditiveAnimationInstance
{
public:
	UE_API FAdditiveAnimationInstance();

	UE_API void Initialise(USkeletalMeshComponent* InSkeletalMeshComponent, UClass* InAnimationBP);
	UE_API void Setup(USkeletalMeshComponent* InBaseComponent, UAnimSequence* InAnimSequence);
	UE_API void UpdateBaseComponent(USkeletalMeshComponent* InBaseComponent);
	UE_API void Stop();
	UE_API void Start();

	UE_API USkeletalMeshComponent* GetComponent() const;
	UE_API USkeletalMeshComponent* GetBaseComponent() const;	

protected:
	USkeletalMeshComponent * SkeletalMeshComponent;
	UAnimSharingAdditiveInstance* AdditiveInstance;
	UAnimSequence* AdditiveAnimationSequence;
	USkeletalMeshComponent* BaseComponent;
	bool bLoopingState;
};

#undef UE_API
