// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimInstance.h"
#include "AnimationSharingInstances.generated.h"

#define UE_API ANIMATIONSHARING_API

class UAnimSequence;

UCLASS(MinimalAPI, transient, Blueprintable)
class UAnimSharingStateInstance : public UAnimInstance
{
	friend class UAnimSharingInstance;

	GENERATED_UCLASS_BODY()

protected:
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Transient, Category = AnimationSharing)
	TObjectPtr<UAnimSequence> AnimationToPlay;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Transient, Category = AnimationSharing)
	float PermutationTimeOffset;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Transient, Category = AnimationSharing)
	float PlayRate;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Transient, Category = AnimationSharing)
	bool bStateBool;

	UFUNCTION(BlueprintCallable, Category = AnimationSharing)
	UE_API void GetInstancedActors(TArray<class AActor*>& Actors);

private:
	uint8 StateIndex;
	uint8 ComponentIndex;
	
	UPROPERTY(Transient)
	TObjectPtr<class UAnimSharingInstance> Instance;
};

UCLASS(transient, Blueprintable)
class UAnimSharingTransitionInstance : public UAnimInstance
{
	friend struct FTransitionBlendInstance;

	GENERATED_UCLASS_BODY()
protected:
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Transient, Category = Transition)
	TWeakObjectPtr<USkeletalMeshComponent> FromComponent;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Transient, Category = Transition)
	TWeakObjectPtr<USkeletalMeshComponent> ToComponent;
	
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Transient, Category = Transition)
	float BlendTime;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Transient, Category = Transition)
	bool bBlendBool;
};


UCLASS(transient, Blueprintable)
class UAnimSharingAdditiveInstance : public UAnimInstance
{
	friend struct FAdditiveAnimationInstance;

	GENERATED_UCLASS_BODY()
protected:
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Transient, Category = Additive)
	TWeakObjectPtr<USkeletalMeshComponent> BaseComponent;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Transient, Category = Additive)
	TWeakObjectPtr<UAnimSequence> AdditiveAnimation;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Transient, Category = Additive)
	float Alpha;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Transient, Category = Additive)
	bool bStateBool;
};

#undef UE_API
