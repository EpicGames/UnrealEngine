// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationSharingTypes.h"

#include "AdditiveAnimationInstance.h"

#include "Components/SkeletalMeshComponent.h"
#include "Engine/World.h"
#include "AnimationSharingManager.generated.h"

#define UE_API ANIMATIONSHARING_API

class UAnimationSharingManager;
class UAnimationSharingSetup;
struct FTransitionBlendInstance;

DECLARE_LOG_CATEGORY_EXTERN(LogAnimationSharing, Log, All);
DECLARE_STATS_GROUP(TEXT("Animation Sharing Manager"), STATGROUP_AnimationSharing, STATCAT_Advanced);

#define DEBUG_MATERIALS (UE_EDITOR && !UE_BUILD_SHIPPING)

DECLARE_DELEGATE_OneParam(FUpdateActorHandle, int32);

class USignificanceManager;
typedef uint32 AnimationSharingDataHandle;

/** Structure which holds data about a currently in progress blend between two states */
struct FBlendInstance
{
	/** Flag whether or not this instance is currently active */
	bool bActive;
	/** Flag whether or not the actor's part of this have been setup as a follower component to the blend actor, this is done so the blend actor atleast ticks once (otherwise it can pop from the previous blend end pose) */
	bool bBlendStarted;
	/** Flag whether or not this instance is blending towards an on-demand state */
	bool bOnDemand;
	/** World time in seconds at which the blend has finished (calculated at start of blend world time + blend duration) */
	float EndTime;
	/** Duration of the blend */
	float BlendTime;
	/** State value to blend from */
	uint8 StateFrom;
	/** State value to blend to */
	uint8 StateTo;

	/** Permutation indices from and to which we are blending, used to ensure we 'forward' the actor to the correct LeaderPoseComponent when finished blending */
	uint32 FromPermutationIndex;
	uint32 ToPermutationIndex;

	/** Actor used for blending between the two states */
	FTransitionBlendInstance* TransitionBlendInstance;
	/** Indices of actors who are set up as followers to BlendActor's main skeletal mesh component */
	TArray<uint32> ActorIndices;

	/** Optional index into OnDemandInstances from which we are blending */
	uint32 FromOnDemandInstanceIndex;
	/** Optional index into OnDemandInstance to which we are blending */
	uint32 ToOnDemandInstanceIndex;
};

/** Structure which holds data about a currently running on-demand state animation instance */
struct FOnDemandInstance
{
	/** Flag whether or not instance is active*/
	bool bActive;
	bool bBlendActive;

	/** Flag whether or not the component should be 'returned' to the state they were in before the on-demand animation */
	bool bReturnToPreviousState;

	/** State value which is active */
	uint8 State;

	/** State value which the components should be set to when the on-demand animation has finished playing (used when !bReturnToPreviousState) */
	uint8 ForwardState;

	/** Time at which this instance was started */
	float StartTime;

	/** Time at which this on demand instance should blend out into the 'next' state the actor is in */
	float StartBlendTime;

	/** World time in seconds at which the animation has finished playing (calculated at start of blend world time + animation sequence length) */
	float EndTime;

	/** Index into Components array for the current state data which is used for playing the animation*/
	uint32 UsedPerStateComponentIndex;

	/** Permutation index that we are blending to before the end of the animation */
	uint32 BlendToPermutationIndex;

	/** Indices of actors who are set up as followers to the skeletal mesh component running the animation */
	TArray<uint32> ActorIndices;
};

struct FAdditiveInstance
{
	/** Flag whether or not instance is active */
	bool bActive;

	/** State index this instance is running */
	uint8 State;

	/** Time at which this instance finishes */
	float EndTime;

	/** Current actor indices as part of this instance */
	uint32 ActorIndex;
	
	/** Skeletal mesh component on which the additive animation is applied */
	USkeletalMeshComponent* BaseComponent;

	/** Actor used for playing the additive animation */
	FAdditiveAnimationInstance* AdditiveAnimationInstance;

	/** Index into Components array for the current state data which is used for playing the animation*/
	uint32 UsedPerStateComponentIndex;
};

template <typename InstanceType>
struct FInstanceStack
{
	~FInstanceStack()
	{
		for (InstanceType* Instance : AvailableInstances)
		{
			delete Instance;
		}
		AvailableInstances.Empty();

		for (InstanceType* Instance : InUseInstances)
		{
			delete Instance;
		}
		InUseInstances.Empty();
	}


	/** Return whether instance are available */
	bool InstanceAvailable() const
	{
		return AvailableInstances.Num() != 0;
	}

	/** Get an available instance */
	InstanceType* GetInstance()
	{
		InstanceType* Instance = nullptr;
		if (AvailableInstances.Num())
		{
			Instance = AvailableInstances.Pop(EAllowShrinking::No);
			InUseInstances.Add(Instance);
		}

		return Instance;
	}

	/** Return instance back */
	void FreeInstance(InstanceType* Instance)
	{
		AvailableInstances.Add(Instance);
		InUseInstances.RemoveSwap(Instance);
	}

	/** Add a new instance to the 'stack' */
	void AddInstance(InstanceType* Instance)
	{
		AvailableInstances.Add(Instance);
	}

	TArray<InstanceType*> AvailableInstances;
	TArray<InstanceType*> InUseInstances;
};

/** Structure which holds data about a unique state which is linked to an enumeration value defined by the user. The data is populated from the user exposed FAnimationStateEntry */
struct FPerStateData
{
	FPerStateData() : bIsOnDemand(false), bIsAdditive(false), bShouldRandomizeOnDemandAnims(false), BlendTime(0.f), CurrentFrameOnDemandIndex(INDEX_NONE), StateEnumValue(INDEX_NONE), AdditiveAnimationSequence(nullptr) {}

	/** Flag whether or not this state is an on-demand state, this means that we kick off a unique animation when needed */
	bool bIsOnDemand;

	/** Flag whether or not this state is an additive state */
	bool bIsAdditive;

	/** Flag whether or not we should return to the previous state, only used when this state is an on-demand one*/
	bool bReturnToPreviousState;

	/** Flag whether or not ForwardStateValue should be used hwen the animation has finished*/
	bool bShouldForwardToState;

	/* Flag to allow random selection of OnDemand components so we don't always get only the first avaialable component. */
	bool bShouldRandomizeOnDemandAnims;

	/** Duration of blending when blending to this state */
	float BlendTime;

	/** This is (re-)set every frame, and allows for quickly finding an on-demand instance which was setup this frame */
	uint32 CurrentFrameOnDemandIndex;
	/** Number of 'wiggle' frames, this is used when we run out of available entries in Components, if one of the FOnDemandInstance has started NumWiggleFrames ago or earlier,
	it is used instead of a brand new one */
	float WiggleTime;

	/** State value to which the actors part of an FOnDemandInstance should be set to when its animation has finished */
	uint8 ForwardStateValue;
	/** Enum value linked to this state */
	uint8 StateEnumValue;

	/** Animation Sequence that is used for Additive States */
	UAnimSequence* AdditiveAnimationSequence;

	/** Components setup to play animations for this state */
	TArray<USkeletalMeshComponent*> Components;
	/** Bits keeping track which of the components are in-use, in case of On Demand state this is managed by FOnDemandInstance, otherwise we clear and populate the flags each frame */
	TBitArray<> InUseComponentFrameBits;
	TBitArray<> PreviousInUseComponentFrameBits;

	/** Bits keeping track whether or not any of the follower components requires the leader component to tick */
	TBitArray<> FollowerTickRequiredFrameBits;

#if UE_EDITOR
	UE_DEPRECATED(5.1, "This member has been deprecated. Please use FollowerTickRequiredFrameBits")
	TBitArray<> SlaveTickRequiredFrameBits;
#endif // UE_EDITOR

	/** Length of the animations used for an on-demand state, array as it could contain different animation permutations */
	TArray<float> AnimationLengths;

	/** Additive actors data structure */
	FInstanceStack<FAdditiveAnimationInstance> AdditiveInstanceStack;
};

struct FPerComponentData
{
	/** Skeletal mesh component registered for this component */
	TWeakObjectPtr<USkeletalMeshComponent> WeakComponent;
	/** Index to the owning actor (used to index PerActorData) */
	int32 ActorIndex;
};

struct FPerActorData
{
	/** Current state value (used to index PerStateData) */
	uint8 CurrentState;
	/** Previous state value (used to index PerStateData) */
	uint8 PreviousState;
	/** Permutation index (used to index Components array inside of PerStateData) */
	uint8 PermutationIndex;

	/** Flag whether or not we are currently blending */
	bool bBlending;
	/** Flag whether or not we are currently part of an on-demand animation state */
	bool bRunningOnDemand;

	/** Flag whether or not we are currently part of an additive animation state */
	bool bRunningAdditive;

	/** Cached significance value */
	float SignificanceValue;

	/** Flag whether or not this actor requires the leader component to tick */
	bool bRequiresTick;

	/** Index to blend instance which is currently driving this actor's animation */
	uint32 BlendInstanceIndex;
	/** Index to on demand instance which is running accord to our current state (or previous state) */
	uint32 OnDemandInstanceIndex;
	/** Index to additive instance which is running on top of our state */
	uint32 AdditiveInstanceIndex;

	/** Indices of the Components owned by this actor (used to index into PerComponentData) */
	TArray<uint32> ComponentIndices;

	/** Register delegate called when actor is swapped and the handle should be updated */
	FUpdateActorHandle UpdateActorHandleDelegate;	
};

UCLASS(MinimalAPI)
class UAnimSharingInstance : public UObject
{
	GENERATED_BODY()

public:
	// Begin UObject overrides
	UE_API virtual void BeginDestroy() override;
	// End UObject overrides

	/** unregister all actors associated with this sharing instance */
	UE_API void UnregisterAllActors();

	/** This uses the StateProcessor to determine the state index the actor is currently in */
	UE_API uint8 DetermineStateForActor(uint32 ActorIndex, bool& bShouldProcess);

	/** Initial set up of all animation sharing data and states */
	UE_API bool Setup(UAnimationSharingManager* AnimationSharingManager, const FPerSkeletonAnimationSharingSetup& SkeletonSetup, const FAnimationSharingScalability* InScalabilitySettings, uint32 InSkeletonID);

	/** Populates data for a state setup */
	UE_API void SetupState(FPerStateData& StateData, const FAnimationStateEntry& StateEntry, USkeletalMesh* SkeletalMesh, const FPerSkeletonAnimationSharingSetup& SkeletonSetup, uint32 InSkeletonID);

	/** Retrieves a blend instance, this could either mean reusing an already in progress one or a brand new one (if available according to scalability settings)
	returns whether or not the setup was successful */
	UE_API uint32 SetupBlend(uint8 FromState, uint8 ToState, uint32 ActorIndex);

	/** Retrieves a blend instance, and sets up a blend from a currently running On-Demand instance to ToState */
	UE_API uint32 SetupBlendFromOnDemand(uint8 ToState, uint32 OnDemandInstanceIndex, uint32 ActorIndex);

	/** Retrieves a blend instance, and sets up a blend between a currently running On-Demand instance and another one which was started this frame */
	UE_API uint32 SetupBlendBetweenOnDemands(uint8 FromOnDemandInstanceIndex, uint32 ToOnDemandInstanceIndex, uint32 ActorIndex);

	/** Retrieves a blend instance, and setups up a blend to an On-Demand instance from a regular animation state */
	UE_API uint32 SetupBlendToOnDemand(uint8 FromState, uint32 ToOnDemandInstanceIndex, uint32 ActorIndex);

	/** Switches between on-demand instances directly, without blending */
	UE_API void SwitchBetweenOnDemands(uint32 FromOnDemandInstanceIndex, uint32 ToOnDemandInstanceIndex, uint32 ActorIndex);

	/** Retrieves a blend instance, this could either mean reusing an already in progress one or a brand new one (if available according to scalability settings)
	returns an index into OnDemandInstances array or INDEX_NONE if unable to setup an instance */
	UE_API uint32 SetupOnDemandInstance(uint8 StateIndex);

	/** Retrieves an additive instance, these are unique and cannot be reused */
	UE_DEPRECATED(5.8, "SetupAdditiveInstance with state and component index has been deprecated as it causes invalid behaviour, use new signature instead")
	uint32 SetupAdditiveInstance(uint8 StateIndex, uint8 FromState, uint8 StateComponentIndex) { return INDEX_NONE; }
	UE_API uint32 SetupAdditiveInstance(uint8 StateIndex, USkeletalMeshComponent* InBaseComponent);

	/** Retrieves the blend-time for this specific state */
	UE_API float CalculateBlendTime(uint8 StateIndex) const;

	/** Kicks off the blend and on-demand instances at the end of the current frame tick, this sets up the blend instance with the correct components to blend between */
	UE_API void KickoffInstances();

	/** Ticks all currently running blend instances, checks whether or not the blend is finished and forwards the actor/components to the correct animation state */
	UE_API void TickBlendInstances();

	/** Ticks all currently running on-demand instances, this checks whether or not the animation has finished or if we have to start blending out of the state already */
	UE_API void TickOnDemandInstances();

	template <typename InstanceType>
	bool DoAnyActorsRequireTicking(const InstanceType& Instance)
	{
		bool bShouldTick = false;
		for (uint32 ActorIndex : Instance.ActorIndices)
		{
			if (PerActorData[ActorIndex].bRequiresTick)
			{
				bShouldTick = true;
				break;
			}
		}

		return bShouldTick;
	}

	/** Ticks all currently running additive animation instances, this checks whether or not it has finished yet and sets the base-component as the leader component when it has */
	UE_API void TickAdditiveInstances();

	/** Ticks all Actor Data entries and determines their current state, if changed since last tick it will alter their animation accordingly */
	UE_API void TickActorStates();

	/** Ticks various types of debugging data / drawing (not active in shipping build) */
	UE_API void TickDebugInformation();

	/** Ticks all unique animation states, this checks which components are currently used and turns of those which currently don't have any followerss */
	UE_API void TickAnimationStates();

	/** Removal functions which also make sure that indices of other data-structres are correctly remapped */
	UE_API void RemoveBlendInstance(int32 InstanceIndex);
	UE_API void RemoveOnDemandInstance(int32 InstanceIndex);
	UE_API void RemoveAdditiveInstance(int32 InstanceIndex);
	UE_API void RemoveComponent(int32 ComponentIndex);

	/** Removal functions which also make sure the actor is set to the correct leader pose component */
	UE_API void RemoveFromCurrentBlend(int32 ActorIndex);
	UE_API void RemoveFromCurrentOnDemand(int32 ActorIndex);

	/** Frees up a Blend instance and resets its state */
	UE_API void FreeBlendInstance(FTransitionBlendInstance* Instance);

	/**  Frees up an Additive Animation instance and resets it state*/
	UE_API void FreeAdditiveInstance(uint8 StateIndex, FAdditiveAnimationInstance* Instance);

	/** Sets up all components of an actor to be followers of Component */
	UE_API void SetLeaderComponentForActor(uint32 ActorIndex, USkeletalMeshComponent* Component);

	UE_DEPRECATED(5.1, "This function has been deprecated. Please use SetLeaderComponentForActor instead.")
	void SetMasterComponentForActor(uint32 ActorIndex, USkeletalMeshComponent* Component) { SetLeaderComponentForActor(ActorIndex, Component); }

	/** Sets up the correct LeaderPoseComponent for the passed in Component and State indices */
	UE_API void SetupFollowerComponent(uint8 CurrentState, uint32 ActorIndex);

	UE_DEPRECATED(5.1, "This function has been deprecated. Please use SetupFollowerComponent instead.")
	void SetupSlaveComponent(uint8 CurrentState, uint32 ActorIndex) { SetupFollowerComponent(CurrentState, ActorIndex); }

	/** Sets up the correct LeaderPoseComponent according to the state and permutation indices */
	UE_API void SetPermutationFollowerComponent(uint8 StateIndex, uint32 ActorIndex, uint32 PermutationIndex);

	UE_DEPRECATED(5.1, "This function has been deprecated. Please use SetPermutationFollowerComponent instead.")
	void SetPermutationSlaveComponent(uint8 StateIndex, uint32 ActorIndex, uint32 PermutationIndex) { SetPermutationFollowerComponent(StateIndex, ActorIndex, PermutationIndex); }
	
	/** Determines a permutation index for the given actor and state */
	UE_API virtual uint32 DeterminePermutationIndex(uint32 ActorIndex, uint8 State) const;

	/** Marks the component as either used/not-used, this is used to disable ticking of components which are not in use*/
	UE_API void SetComponentUsage(bool bUsage, uint8 StateIndex, uint32 ComponentIndex);

	/** Sets the whether or not any of the follower components are visible */
	UE_API void SetComponentTick(uint8 StateIndex, uint32 ComponentIndex);

	/** Retrieves USkeletalMeshComponent with validation of reachability */
	UE_API USkeletalMeshComponent* GetComponent(const FPerComponentData& ComponentData);

	/** Actors currently registered to be animation driven by the AnimManager using this setup */
	UPROPERTY(VisibleAnywhere, Transient, Category = AnimationSharing)
	TArray<TObjectPtr<AActor>> RegisteredActors;
	
	/** Per actor data, matches RegisteredActors*/
	TArray<FPerActorData> PerActorData;
	/** Per component state data indexed from FPerActorData.ComponentIndices */
	TArray<FPerComponentData> PerComponentData;

	/** Array of unique state data */
	TArray<FPerStateData> PerStateData;

	/** Blend actors data structure */
	FInstanceStack<FTransitionBlendInstance> BlendInstanceStack;

	/** (Blueprint)class instance used for determining the state enum value for each registered actor */
	UPROPERTY(EditAnywhere, Transient, Category = AnimationSharing)
	TObjectPtr<UAnimationSharingStateProcessor> StateProcessor;
	bool bNativeStateProcessor;

	/** weak ptr back to the setup that owns this instance */
	TWeakObjectPtr<const UAnimationSharingSetup> OwnerSetup;

	/** id for skeleton associated with this instance that remains even when setups are removed and added. */	
	uint32 SkeletonID;

	/** Currently running blend instances */
	TArray<FBlendInstance> BlendInstances;
	/** Currently running on-demand instance */
	TArray<FOnDemandInstance> OnDemandInstances;
	/** Currently running additive instances */
	TArray<FAdditiveInstance> AdditiveInstances;

	UPROPERTY(VisibleAnywhere, Transient, Category = AnimationSharing)
	TArray<TObjectPtr<UAnimSequence>> UsedAnimationSequences;

	/** Significance manager used to retrieve AI actor significance values */
	USignificanceManager* SignificanceManager;

	/** Animation sharing manager for the current world */
	UAnimationSharingManager* AnimSharingManager;

	/** Enum class set up by the user to 'describe' the animation states */
	UPROPERTY(VisibleAnywhere, Transient, Category = AnimationSharing)
	TObjectPtr<UEnum> StateEnum;

	/** Actor to which all the running SkeletalMeshComponents used for the sharing are attached to */
	UPROPERTY(VisibleAnywhere, Transient, Category = AnimationSharing)
	TObjectPtr<AActor> SharingActor;

	/** Platform specific scalability settings for this instance */
	FAnimationSharingScalability ScalabilitySettings;

	/** Bounds for the currently used skeletal mesh */
	FVector SkeletalMeshBounds;

	/** Number of animation setups */
	uint32 NumSetups;

	/** Holds the current frame world time */
	float WorldTime;

private:

#if UE_BUILD_DEVELOPMENT
	UE_API void GetAnimAssetNameAndData(const uint32 StateIdx, const uint32 CompIdx, FString& AnimAssetName, float& AnimTime, float& AnimLength) const;
	UE_API FString GetDebugStateString(uint8 State, const uint32 CompIdx, uint32 OnDemandInstanceIndex) const;
#endif // #if UE_BUILD_DEVELOPMENT

};

USTRUCT()
struct FTickAnimationSharingFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	FTickAnimationSharingFunction() : Manager(nullptr) {}

	// Begin FTickFunction overrides
	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	virtual FString DiagnosticMessage() override;
	virtual FName DiagnosticContext(bool bDetailed) override;
	// End FTickFunction overrides
	
	class UAnimationSharingManager* Manager;
};


template<>
struct TStructOpsTypeTraits<FTickAnimationSharingFunction> : public TStructOpsTypeTraitsBase2<FTickAnimationSharingFunction>
{
	enum
	{
		WithCopy = false
	};
};

UCLASS(MinimalAPI, config = Engine, defaultconfig)
class UAnimationSharingManager : public UObject
{
	GENERATED_BODY()

public:
	// Begin UObject overrides
	UE_API virtual void BeginDestroy() override;
	UE_API virtual UWorld* GetWorld()const override;
	// End UObject overrides

	/** Returns the AnimationSharing Manager, nullptr if none was set up */
	UFUNCTION(BlueprintCallable, Category = AnimationSharing, meta = (WorldContext = "WorldContextObject"))
	static UE_API UAnimationSharingManager* GetAnimationSharingManager(UObject* WorldContextObject);

	/** Create an Animation Sharing Manager using the provided Setup */
	UFUNCTION(BlueprintCallable, Category = AnimationSharing, meta = (WorldContext = "WorldContextObject"))
	static UE_API bool CreateAnimationSharingManager(UObject* WorldContextObject, const UAnimationSharingSetup* Setup);

	/** Register an Actor with this Animation Sharing manager, according to the SharingSkeleton */
	UFUNCTION(BlueprintCallable, Category = AnimationSharing, meta = (DisplayName = "Register Actor"))
	UE_API bool RegisterActorWithSkeletonBP(AActor* InActor, const USkeleton* SharingSkeleton);

	/** Returns whether or not the animation sharing is enabled */
	UFUNCTION(BlueprintPure, Category = AnimationSharing)
	static UE_API bool AnimationSharingEnabled();

	/** Returns the AnimationSharing Manager for a specific UWorld, nullptr if none was set up */
	static UE_API UAnimationSharingManager* GetManagerForWorld(UWorld* InWorld);

	/** Registers actor with the animation sharing system */
	UE_API bool RegisterActor(AActor* InActor, FUpdateActorHandle Delegate);

	/** Registers actor with the animation sharing system according to the SharingSkeleton's sharing setup (if available) */
	UE_API bool RegisterActorWithSkeleton(AActor* InActor, const USkeleton* SharingSkeleton, FUpdateActorHandle Delegate);

	/** Unregisters actor with the animation sharing system */
	UE_API bool UnregisterActor(AActor* InActor);

	/** Update cached significance for registered actor */
	UE_API void UpdateSignificanceForActorHandle(uint32 InHandle, float InValue);

	/** Ensures all actor data is cleared */
	UE_API void ClearActorData();

	/** Ensures all currently registered actors are removed */
	UE_API void UnregisterAllActors();	

	/** Sets the visibility of currently used Leader Pose Components */
	UE_API void SetLeaderComponentsVisibility(bool bVisible);
	
	UE_DEPRECATED(5.1, "This member has been deprecated. Please use SetLeaderComponentsVisibility")
	void SetMasterComponentsVisibility(bool bVisible) { SetLeaderComponentsVisibility(bVisible); }

	/** Initialize sharing data structures */
	UE_API void Initialise(const UAnimationSharingSetup* InSetup);

	/** Add an animation sharing setup to the manager */
	UE_API void AddSetup(const UAnimationSharingSetup* InSetup);

	/** Remove an animation sharing setup from the manager */
	UE_API void RemoveSetup(const UAnimationSharingSetup* InSetup);

	/** Returns current scalability settings, with optional setup parameter to return scalability settings specific to an animation sharing setup */
	UE_API const FAnimationSharingScalability& GetScalabilitySettings(const UAnimationSharingSetup* InSetup = nullptr) const;

	UE_API void Tick(float DeltaTime);
	UE_API FTickAnimationSharingFunction& GetTickFunction();

	/** Checks whether the supplied actor is registered and all its data is valid */
	UE_API bool CheckDataForActor(AActor* InActor) const;

	/** Log out internal data */
	UE_API void LogData() const;

	static uint32 GetInvalidActorHandle() { return InvalidActorHandle; };

protected:
	
	/** Populates all data required for a Skeleton setup */
	UE_DEPRECATED(5.7, "Replaced by SetupPerSkeletonData version below with extra parameters.")
	UE_API void SetupPerSkeletonData(const FPerSkeletonAnimationSharingSetup& SkeletonSetup);
	UE_API void SetupPerSkeletonData(const FPerSkeletonAnimationSharingSetup& SkeletonSetup, const UAnimationSharingSetup* SharingSetup, const FAnimationSharingScalability& ScalabilitySettingsForSkeleton);

	UE_API virtual UAnimSharingInstance* CreateAnimSharingInstance();

	/** Dealing with Actor data and handles */
	UE_API uint32 CreateActorHandle(uint8 SkeletonID, uint32 ActorIndex) const;
	UE_API uint8 GetSkeletonIndexFromHandle(uint32 InHandle) const;
	UE_API uint32 GetActorIndexFromHandle(uint32 InHandle) const;
	UE_API FPerActorData* GetActorDataByHandle(uint32 InHandle);
	
protected:

	/** Array of unique skeletons, matches PerSkeletonData array entries*/
	UPROPERTY(Transient)
	TArray<TObjectPtr<const USkeleton>> Skeletons;

	/** Sharing data required for the unique Skeleton setups */
	UPROPERTY(VisibleAnywhere, Transient, Category = AnimationSharing)
	TArray<TObjectPtr<UAnimSharingInstance>> PerSkeletonData;

	/** map from skeleton id to skeleton index to speed up accesssing from actor handles */
	TMap< uint8, uint8 > SkeletonIDToSkeletonIndexMap;

	/** Max skeleton id matches the range supported in actor handles */
	static const uint32 MaxSkeletonID = 0xFF;
	static const uint32 MaxSkeletonIndex = 0xFF;
	static const uint32 InvalidActorHandle = INDEX_NONE;

	/** Skeleton ID for next added skeleton setup */
	uint32 NextSkeletonID = 0;
	
	/** Pointers to animation sharing setups added to manager */
	TArray< TWeakObjectPtr<const UAnimationSharingSetup> > AnimationSharingSetups;

	/** Platform specific scalability settings (default version, setting from a setup will generally be used instead) */
	FAnimationSharingScalability ScalabilitySettings;

	/** Tick function for this manager */
	FTickAnimationSharingFunction TickFunction;

public:
#if DEBUG_MATERIALS 	
	static UE_API TArray<TObjectPtr<UMaterialInterface>> DebugMaterials;	
#endif
	static UE_API void SetDebugMaterial(USkeletalMeshComponent* Component, uint8 State);
	static UE_API void SetDebugMaterialForActor(UAnimSharingInstance* Data, uint32 ActorIndex, uint8 State);

#if WITH_EDITOR
	static UE_API FName GetPlatformName();
#endif
};

#undef UE_API
