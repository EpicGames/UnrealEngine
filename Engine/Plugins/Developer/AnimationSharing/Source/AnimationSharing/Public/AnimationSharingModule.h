// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Engine/World.h"

#define UE_API ANIMATIONSHARING_API

class UAnimationSharingManager;
class UAnimationSharingSetup;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAnimationSharingManagerCreated, UAnimationSharingManager*, const UWorld*);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAnimationSharingManagerSetupAdded, UAnimationSharingManager*, const UWorld*);

class FAnimSharingModule : public FDefaultModuleImpl, public FGCObject
{
public:
	// Begin IModuleInterface overrides
	UE_API virtual void StartupModule() override;
	// End IModuleInterface overrides

	// Begin FGCObject overrides
	UE_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FAnimSharingModule");
	}
	// End FGCObject overrides

	FORCEINLINE static UAnimationSharingManager* Get(const UWorld* World)
	{
		return WorldAnimSharingManagers.FindRef(World);
	}

	FORCEINLINE static FOnAnimationSharingManagerCreated& GetOnAnimationSharingManagerCreated()
	{
		return OnAnimationSharingManagerCreated;
	}

	FORCEINLINE static FOnAnimationSharingManagerSetupAdded& GetOnAnimationSharingManagerSetupAdded()
	{
		return OnAnimationSharingManagerSetupAdded;
	}

	/** Creates an animation sharing manager for the given UWorld (must be a Game World) */
	static UE_API bool CreateAnimationSharingManager(UWorld* InWorld, const UAnimationSharingSetup* Setup);
	
	/** Creates an animation sharing manager for the given UWorld (must be a Game World) - with 
		custom func that is responsible for creation and initialisation */
	using FCreateAnimationSharingManagerFunc = TFunction<UAnimationSharingManager*(UWorld* InWorld)>;
	static UE_API bool CreateAnimationSharingManager(UWorld* InWorld, FCreateAnimationSharingManagerFunc CreateAnimationSharingManagerFunc);

private:	
	static UE_API void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);
	static UE_API TMap<const UWorld*, TObjectPtr<UAnimationSharingManager>> WorldAnimSharingManagers;

	static UE_API FOnAnimationSharingManagerCreated OnAnimationSharingManagerCreated;
	static UE_API FOnAnimationSharingManagerSetupAdded OnAnimationSharingManagerSetupAdded;
};

#undef UE_API
