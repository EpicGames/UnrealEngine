// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationSharingTypes.h"

#include "AnimationSharingSetup.generated.h"

/**
 * The Animation Sharing Setup asset contains all the information that will be shared across the specified Actors
 */
UCLASS(MinimalAPI, hidecategories = Object, Blueprintable, config = Engine)
class UAnimationSharingSetup : public UObject
{
	GENERATED_UCLASS_BODY()
public:

	UPROPERTY(EditAnywhere, config, Category = AnimationSharing)
	TArray<FPerSkeletonAnimationSharingSetup> SkeletonSetups;

	UPROPERTY(EditAnywhere, config, Category = AnimationSharing)
	FAnimationSharingScalability ScalabilitySettings;
};
