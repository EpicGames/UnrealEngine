// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/PerPlatformProperties.h"
#include "Templates/SubclassOf.h"

#include "AnimationSharingTypes.generated.h"

#define UE_API ANIMATIONSHARING_API

class UAnimSequence;
class UAnimSharingAdditiveInstance;
class UAnimSharingStateInstance;
class UAnimSharingTransitionInstance;
class USkeletalMesh;
class USkeleton;

USTRUCT()
struct FAnimationSetup
{
	GENERATED_BODY()
public:
	FAnimationSetup() : AnimSequence(nullptr), AnimBlueprint(nullptr), NumRandomizedInstances(1), Enabled(true) {}
	
	/** Animation Sequence to play for this particular setup */
	UPROPERTY(EditAnywhere, Category = AnimationSharing)
	TObjectPtr<UAnimSequence> AnimSequence;

	/** Animation blueprint to use for playing back the Animation Sequence */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = AnimationSharing)
	TSubclassOf<UAnimSharingStateInstance> AnimBlueprint;	

	/** The number of randomized instances created from this setup, it will create instance with different start time offsets (Length / Number of Instance) * InstanceIndex */
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (ClampMin = "1", UIMin = "1"))
	FPerPlatformInt NumRandomizedInstances;

	/** Whether or not this setup is enabled for specific platforms */
	UPROPERTY(EditAnywhere, Category = AnimationSharing)
	FPerPlatformBool Enabled;
};

USTRUCT()
struct FAnimationStateEntry
{
	GENERATED_BODY()
public:
	FAnimationStateEntry() : State(0), bOnDemand(false), bRandomizeOnDemandAnims(false), bAdditive(false), BlendTime(0.f), bReturnToPreviousState(false), bSetNextState(false), NextState(0), MaximumNumberOfConcurrentInstances(1), WiggleTimePercentage(0.1f), bRequiresCurves(false) {}

	/** Enum value linked to this state */
	UPROPERTY(EditAnywhere, Category = AnimationSharing)
	uint8 State;

	/** Per state animation setup */
	UPROPERTY(EditAnywhere, Category = AnimationSharing)
	TArray<FAnimationSetup> AnimationSetups;

	/** Flag whether or not this state is an on-demand state, this means that we kick off a unique animation when needed */
	UPROPERTY(EditAnywhere, Category = AnimationSharing)
	bool bOnDemand;

	/* Flag that determines whether or not we attempt to randomize the selection of OnDemand animations, if the total number of components for the OnDemand state is more than 1. */
	UPROPERTY(EditAnywhere, Category = AnimSharing, meta = (EditCondition = "bOnDemand"))
	bool bRandomizeOnDemandAnims;

	/** Whether or not this state is an additive state */
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (EditCondition = "bOnDemand"))
	bool bAdditive;

	/** Duration of blending when blending to this state */
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (EditCondition = "!bAdditive", UIMin = "0.0", ClampMin = "0.0"))
	float BlendTime;

	/** Flag whether or not we should return to the previous state, only used when this state is an on-demand one*/
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (EditCondition = "bOnDemand"))
	bool bReturnToPreviousState;
		
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (EditCondition = "bOnDemand"))
	bool bSetNextState;

	/** State value to which the actors part of an on demand state should be set to when its animation has finished */
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (EditCondition = "bSetNextState"))
	uint8 NextState;

	/** Number of instances that will be created for this state (platform-specific) */
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta=(EditCondition="bOnDemand", ClampMin = "0", UIMin = "0"))
	FPerPlatformInt MaximumNumberOfConcurrentInstances;

	/** Percentage of 'wiggle' frames, this is used when we run out of available entries in Components, if one of the on-demand animations has started SequenceLength * WiggleFramePercentage ago or earlier,
	it is used instead of a brand new one */
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (EditCondition = "bOnDemand", UIMin="0.0", UIMax="1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float WiggleTimePercentage;
	
	/** Whether or not this animation requires curves or morphtargets to function correctly for follower components */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = AnimationSharing)
	bool bRequiresCurves;
};

UCLASS(MinimalAPI, Blueprintable)
class UAnimationSharingStateProcessor : public UObject
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintNativeEvent, Category = AnimationSharing)
	UE_API void ProcessActorState(int32& OutState, AActor* InActor, uint8 CurrentState, uint8 OnDemandState, bool& bShouldProcess);
	virtual void ProcessActorState_Implementation(int32& OutState, AActor* InActor, uint8 CurrentState, uint8 OnDemandState, bool& bShouldProcess)
	{
		ProcessActorState_Internal(OutState, InActor, CurrentState, OnDemandState, bShouldProcess);
	}

	UFUNCTION(BlueprintNativeEvent, Category = AnimationSharing)
	UE_API UEnum* GetAnimationStateEnum();
	virtual UEnum* GetAnimationStateEnum_Implementation()
	{
		return GetAnimationStateEnum_Internal();
	}

	virtual void ProcessActorStates(TArray<int32>& OutStates, const TArray<AActor*>& InActors, const TArray<uint8>& CurrentStates, const TArray<uint8>& OnDemandStates, TArray<bool>& ShouldProcessFlags)
	{
		for (int32 Index = 0; Index < InActors.Num(); ++Index)
		{
			ProcessActorState(OutStates[Index], InActors[Index], CurrentStates[Index], OnDemandStates[Index], ShouldProcessFlags[Index]);
		}
	}

	UPROPERTY(EditAnywhere, Category = AnimationSharing)
	TSoftObjectPtr<UEnum> AnimationStateEnum;
protected:
	virtual UEnum* GetAnimationStateEnum_Internal() { return AnimationStateEnum.LoadSynchronous(); }
	virtual void ProcessActorState_Internal(int32& OutState, AActor* InActor, uint8 CurrentState, uint8 OnDemandState, bool& bShouldProcess) {}
};

USTRUCT()
struct FPerSkeletonAnimationSharingSetup
{
	GENERATED_BODY()
public:
	FPerSkeletonAnimationSharingSetup() : Skeleton(nullptr), SkeletalMesh(nullptr), BlendAnimBlueprint(nullptr), AdditiveAnimBlueprint(nullptr), StateProcessorClass(nullptr), bEnableMaterialParameterCaching(false){}

	/** Skeleton compatible with the animation sharing setup */
	UPROPERTY(EditAnywhere, Category = AnimationSharing)
	TObjectPtr<USkeleton> Skeleton;

	/** Skeletal mesh used to setup skeletal mesh components */
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (DisallowedClasses = "/Script/ApexDestruction.DestructibleMesh"))
	TObjectPtr<USkeletalMesh> SkeletalMesh;

	/** Animation blueprint used to perform the blending between states */
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (DisplayName="Animation Blueprint for Blending"))
	TSubclassOf<UAnimSharingTransitionInstance> BlendAnimBlueprint;
	
	/** Animation blueprint used to apply additive animation on top of other states */
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (DisplayName = "Animation Blueprint for Additive Animation"))
	TSubclassOf<UAnimSharingAdditiveInstance> AdditiveAnimBlueprint;

	/** Interface class used when determining which state an actor is in */
	UPROPERTY(EditAnywhere, Category = AnimationSharing)
	TSubclassOf<UAnimationSharingStateProcessor> StateProcessorClass;

	/** Whether or not to enable material parameter caching on the skeletal mesh components created for each state */
	UPROPERTY(EditAnywhere, Category = AnimationSharing)
	bool bEnableMaterialParameterCaching;

	/** Definition of different animation states */
	UPROPERTY(EditAnywhere, Category = AnimationSharing)
	TArray<FAnimationStateEntry> AnimationStates;
};

USTRUCT()
struct FAnimationSharingScalability
{
	GENERATED_BODY()
public:
	FAnimationSharingScalability() : UseBlendTransitions(true), BlendSignificanceValue(0.0f), MaximumNumberConcurrentBlends(1) {}

	/** Flag whether or not to use blend transitions between states */
	UPROPERTY(EditAnywhere, Category = AnimationSharing)
	FPerPlatformBool UseBlendTransitions;

	/** Significance value tied to whether or not a transition should be blended */
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (ClampMin="0.0", UIMin="0.0"))
	FPerPlatformFloat BlendSignificanceValue;

	/** Maximum number of blends which can be running concurrently */
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (ClampMin = "1", UIMin = "1"))
	FPerPlatformInt MaximumNumberConcurrentBlends;

	/** Significance value tied to whether or not the leader pose components should be ticking */
	UPROPERTY(EditAnywhere, Category = AnimationSharing, meta = (ClampMin = "0.0", UIMin = "0.0"))
	FPerPlatformFloat TickSignificanceValue;

	/** Whether or not curves are required during blends */
	UPROPERTY(EditAnywhere, Category = AnimationSharing)
	FPerPlatformBool RequireCurvesDuringBlends;
};

#undef UE_API
