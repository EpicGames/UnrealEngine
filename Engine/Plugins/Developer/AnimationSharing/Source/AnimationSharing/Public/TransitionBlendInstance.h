// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

#define UE_API ANIMATIONSHARING_API

class UClass;
class USkeletalMeshComponent;

class UAnimSharingTransitionInstance;

struct FTransitionBlendInstance
{
public:	
	UE_API FTransitionBlendInstance();
	UE_API void Initialise(USkeletalMeshComponent* InSkeletalMeshComponent, UClass* InAnimationBP);
	UE_API void Setup(USkeletalMeshComponent* InFromComponent, USkeletalMeshComponent* InToComponent, float InBlendTime);
	UE_API void Stop();

	UE_API USkeletalMeshComponent* GetComponent() const;
	UE_API USkeletalMeshComponent* GetToComponent() const;
	UE_API USkeletalMeshComponent* GetFromComponent() const;

protected:
	USkeletalMeshComponent * SkeletalMeshComponent;
	UAnimSharingTransitionInstance* TransitionInstance;
	USkeletalMeshComponent* FromComponent;
	USkeletalMeshComponent* ToComponent;
	float BlendTime;
	bool bBlendState;
};

#undef UE_API
