// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AnimationSharingEd : ModuleRules
{
	public AnimationSharingEd(ReadOnlyTargetRules Target) : base(Target)
	{
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
                "UnrealEd",
                "AssetTools",
                "AnimationSharing",
                "Slate",
                "SlateCore",
                "PropertyEditor",
            }
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
