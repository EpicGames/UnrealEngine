// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationSharingEdModule.h"
#include "AssetTypeActions_AnimationSharingSetup.h"
#include "PropertyEditorModule.h"
#include "SetupDetailsViewCustomizations.h"

IMPLEMENT_MODULE(FAnimSharingEdModule, AnimationSharingEd );

void FAnimSharingEdModule::StartupModule()
{
	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();

	IAssetTools& AssetTools = AssetToolsModule.Get();
	AssetAction = new FAssetTypeActions_AnimationSharingSetup();
	AssetTools.RegisterAssetTypeActions(MakeShareable(AssetAction));

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	{
		PropertyModule.RegisterCustomPropertyTypeLayout("PerSkeletonAnimationSharingSetup", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPerSkeletonAnimationSharingSetupCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("AnimationStateEntry", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimationStateEntryCustomization::MakeInstance));		

		PropertyModule.RegisterCustomPropertyTypeLayout("AnimationSetup", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimationSetupCustomization::MakeInstance));
	}
}

void FAnimSharingEdModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
		IAssetTools& AssetTools = AssetToolsModule.Get();
		AssetTools.UnregisterAssetTypeActions(AssetAction->AsShared());

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		{
			PropertyModule.UnregisterCustomPropertyTypeLayout("PerSkeletonAnimationSharingSetup");
			PropertyModule.UnregisterCustomPropertyTypeLayout("AnimationStateEntry");
			PropertyModule.UnregisterCustomPropertyTypeLayout("AnimationSetup");
		}
	}
}

