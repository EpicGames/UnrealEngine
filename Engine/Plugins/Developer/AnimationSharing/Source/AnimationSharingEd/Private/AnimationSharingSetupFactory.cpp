// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationSharingSetupFactory.h"
#include "AnimationSharingSetup.h"
#include "AssetTypeCategories.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationSharingSetupFactory)

UAnimationSharingSetupFactory::UAnimationSharingSetupFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAnimationSharingSetup::StaticClass();
	bCreateNew = true;
}

UObject* UAnimationSharingSetupFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAnimationSharingSetup>(InParent, Class, Name, Flags);
}

uint32 UAnimationSharingSetupFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}

