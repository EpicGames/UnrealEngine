// Copyright Epic Games, Inc. All Rights Reserved.

#include "SetupDetailsViewCustomizations.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/STextComboBox.h"
#include "AnimationSharingTypes.h"

#define LOCTEXT_NAMESPACE "AnimationSharingSetupCustomization"

TSharedRef<IPropertyTypeCustomization> FPerSkeletonAnimationSharingSetupCustomization::MakeInstance()
{
	return MakeShareable(new FPerSkeletonAnimationSharingSetupCustomization());
}

void FPerSkeletonAnimationSharingSetupCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	SkeletonPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPerSkeletonAnimationSharingSetup, Skeleton));
	if (SkeletonPropertyHandle.IsValid())
	{		
		HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				// Show the name of the asset or actor
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.Text(this, &FPerSkeletonAnimationSharingSetupCustomization::GetSkeletonName)
			]
		];		
	}
}

void FPerSkeletonAnimationSharingSetupCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	const TArray<FName> SkeletonDisabledProperties = 
	{
		GET_MEMBER_NAME_CHECKED(FPerSkeletonAnimationSharingSetup, SkeletalMesh),
		GET_MEMBER_NAME_CHECKED(FPerSkeletonAnimationSharingSetup, StateProcessorClass),
		GET_MEMBER_NAME_CHECKED(FPerSkeletonAnimationSharingSetup, BlendAnimBlueprint),
		GET_MEMBER_NAME_CHECKED(FPerSkeletonAnimationSharingSetup, AdditiveAnimBlueprint)
	};

	const TArray<FName> EnumDisabledProperties =
	{
		GET_MEMBER_NAME_CHECKED(FPerSkeletonAnimationSharingSetup, AnimationStates)
	};

	TSharedPtr<IPropertyHandle> ProcessorProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPerSkeletonAnimationSharingSetup, 
		StateProcessorClass));

	void* StructPtr = nullptr;	
	StructPropertyHandle->GetValueData(StructPtr);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		IDetailPropertyRow& Property = StructBuilder.AddProperty(ChildHandle);

		const FName& PropertyName = ChildHandle->GetProperty()->GetFName();

		/** Properties disabled by an invalid USkeleton */
		if (SkeletonDisabledProperties.Contains(PropertyName))
		{
			Property.IsEnabled(TAttribute<bool>::Create([this]() -> bool
			{
				UObject* Object = nullptr;
				return (SkeletonPropertyHandle->GetValue(Object) == FPropertyAccess::Success) && (Object != nullptr);
			}));
		}

		/** Properties disabled by invalid UEnum class */
		if (EnumDisabledProperties.Contains(PropertyName))
		{
			Property.IsEnabled(TAttribute<bool>::Create([ProcessorProperty]() -> bool
			{
				const UEnum* EnumClass = GetStateEnumClass(ProcessorProperty);
				return (EnumClass != nullptr);
			}));
		}

		/** Disable additive Anim BP property if there aren't any additive states in the setup */
		if (StructPtr)
		{
			if (PropertyName == GET_MEMBER_NAME_CHECKED(FPerSkeletonAnimationSharingSetup, AdditiveAnimBlueprint))
			{
				Property.IsEnabled(TAttribute<bool>::Create([StructPtr]() -> bool
				{
					FPerSkeletonAnimationSharingSetup* SetupStruct = (FPerSkeletonAnimationSharingSetup*)StructPtr;
					if (SetupStruct)
					{
						const bool bContainsAdditive = SetupStruct->AnimationStates.ContainsByPredicate([](FAnimationStateEntry& Entry) -> bool
						{
							return Entry.bAdditive;
						});

						return bContainsAdditive;
					}

					return false;
				}));
			}
		}		
	}
}

FText FPerSkeletonAnimationSharingSetupCustomization::GetSkeletonName() const
{
	UObject* SkeletonObject;
	FPropertyAccess::Result Result = SkeletonPropertyHandle->GetValue(SkeletonObject);

	FText Name = LOCTEXT("None", "None");
	if (Result == FPropertyAccess::Success)
	{
		if (SkeletonObject)
		{
			Name = FText::AsCultureInvariant(SkeletonObject->GetName());
		}		
	}	

	return Name;
}

TSharedRef<IPropertyTypeCustomization> FAnimationStateEntryCustomization::MakeInstance()
{
	return MakeShareable(new FAnimationStateEntryCustomization());
}

void FAnimationStateEntryCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> ParentHandle = PropertyHandle->GetParentHandle()->GetParentHandle();

	// We make the assumption here that the parent handle is the array part of the FPerSkeletonAnimationSharingSetup	
	ProcessorPropertyHandle = ParentHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPerSkeletonAnimationSharingSetup, StateProcessorClass));
	
	StatePropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimationStateEntry, State));
	if (StatePropertyHandle.IsValid())
	{		
		HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				// Show the name of the asset or actor
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.Text(this, &FAnimationStateEntryCustomization::GetStateName, StatePropertyHandle)
			]
		];		
	}
}

void FAnimationStateEntryCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	const TArray<FName> OnDemandProperties = { GET_MEMBER_NAME_CHECKED(FAnimationStateEntry, bReturnToPreviousState), GET_MEMBER_NAME_CHECKED(FAnimationStateEntry, bSetNextState), GET_MEMBER_NAME_CHECKED(FAnimationStateEntry, NextState), GET_MEMBER_NAME_CHECKED(FAnimationStateEntry, WiggleTimePercentage) };

	const TArray<FName> EnumProperties = { GET_MEMBER_NAME_CHECKED(FAnimationStateEntry, State), GET_MEMBER_NAME_CHECKED(FAnimationStateEntry, NextState) };

	TSharedPtr<IPropertyHandle> OnDemandHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimationStateEntry, bOnDemand));
	TSharedPtr<IPropertyHandle> AdditiveHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimationStateEntry, bAdditive));

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		
		// Hide any on-demand settings when either the state is not an on-demand or it is but an additive as well
		TAttribute<EVisibility> VisibilityAttribute = TAttribute<EVisibility>::Create([OnDemandHandle, AdditiveHandle]() -> EVisibility
		{
			if (OnDemandHandle.IsValid() && AdditiveHandle.IsValid())
			{
				bool bOnDemandValue = false;
				OnDemandHandle->GetValue(bOnDemandValue);

				bool bAdditiveValue = false;
				AdditiveHandle->GetValue(bAdditiveValue);

				return bOnDemandValue && !bAdditiveValue ? EVisibility::Visible : EVisibility::Collapsed;
			}

			return EVisibility::Visible;
		});
				
		if (EnumProperties.Contains(ChildHandle->GetProperty()->GetFName()))
		{
			FDetailWidgetRow& WidgetRow = CreateEnumSelectionWidget(ChildHandle, StructBuilder);
			if (OnDemandProperties.Contains(ChildHandle->GetProperty()->GetFName()))
			{
				WidgetRow.Visibility(VisibilityAttribute);
			}
		}
		else
		{
			IDetailPropertyRow& PropertyRow = StructBuilder.AddProperty(ChildHandle);
			if (OnDemandProperties.Contains(ChildHandle->GetProperty()->GetFName()))
			{
				PropertyRow.Visibility(VisibilityAttribute);
			}
		}		
	}
}

FText FAnimationStateEntryCustomization::GetStateName(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	uint8 EnumValue;
	FPropertyAccess::Result Result = PropertyHandle->GetValue(EnumValue);

	FText Name = LOCTEXT("None", "None");
	if (Result == FPropertyAccess::Success)
	{
		if (UEnum* EnumClass = GetStateEnumClass(ProcessorPropertyHandle))
		{
			Name = EnumClass->GetDisplayNameTextByIndex(EnumValue);
		}
		else
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("EnumIndex"), EnumValue);
			Name = FText::Format(LOCTEXT("EnumIndexValue", "Enum Index {EnumIndex}"), Args);
		}
	}

	return Name;
}

FDetailWidgetRow& FAnimationStateEntryCustomization::CreateEnumSelectionWidget(TSharedRef<IPropertyHandle> ChildHandle, IDetailChildrenBuilder& StructBuilder)
{
	GenerateEnumComboBoxItems();

	TSharedPtr<FString> CurrentlySelected = GetSelectedEnum(ChildHandle);

	FDetailWidgetRow& DetailRow = StructBuilder.AddCustomRow(LOCTEXT("EnumStateSearchLabel", "State"))
	.NameContent()
	[
		ChildHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)		
		+SHorizontalBox::Slot()
		[
			SNew(STextComboBox)
			.OptionsSource(&ComboBoxItems)
			.InitiallySelectedItem(CurrentlySelected)
			.OnSelectionChanged(this, &FAnimationStateEntryCustomization::SelectedEnumChanged, ChildHandle)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
		]
	];

	return DetailRow;
}

const TArray<TSharedPtr<FString>> FAnimationStateEntryCustomization::GetComboBoxSourceItems() const
{
	TArray<TSharedPtr<FString>> Items;

	if (UEnum* EnumClass = GetStateEnumClass(ProcessorPropertyHandle))
	{
		const int32 NumEnums = EnumClass->NumEnums();
		for (int32 Index = 0; Index < NumEnums; ++Index)
		{
			Items.Add(MakeShareable(new FString(EnumClass->GetDisplayNameTextByIndex(Index).ToString())));
		}
	}

	return Items;
}

const TSharedPtr<FString> FAnimationStateEntryCustomization::GetSelectedEnum(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	const TSharedPtr<FString>* StringPtr = ComboBoxItems.FindByPredicate([this, PropertyHandle](TSharedPtr<FString> SharedString) {
		return *SharedString == GetStateName(PropertyHandle).ToString();
	});

	return StringPtr != nullptr ? *StringPtr : MakeShareable(new FString(GetStateName(PropertyHandle).ToString()));
}

void FAnimationStateEntryCustomization::SelectedEnumChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo, TSharedRef<IPropertyHandle> PropertyHandle)
{
	if (UEnum* EnumClass = GetStateEnumClass(ProcessorPropertyHandle))
	{
		if (Selection && SelectInfo != ESelectInfo::Type::Direct)
		{
			const uint8 NewEnumValue = IntCastChecked<uint8>(EnumClass->GetValueByIndex(ComboBoxItems.IndexOfByKey(Selection)));
			PropertyHandle->SetValue(NewEnumValue);
		}

	}
}

void FAnimationStateEntryCustomization::GenerateEnumComboBoxItems()
{
	if (UEnum* EnumClass = GetStateEnumClass(ProcessorPropertyHandle))
	{
		if (EnumClass != CachedComboBoxEnumClass)
		{
			ComboBoxItems.Empty();
			const int32 NumEnums = EnumClass->NumEnums();
			for (int32 Index = 0; Index < NumEnums; ++Index)
			{
				ComboBoxItems.Add(MakeShareable(new FString(EnumClass->GetDisplayNameTextByIndex(Index).ToString())));
			}

			CachedComboBoxEnumClass = EnumClass;
		}
	}
}

UEnum* GetStateEnumClass(const TSharedPtr<IPropertyHandle>& InProperty)
{
	UObject* EnumObject = nullptr;
	if (InProperty.IsValid())
	{
		UObject* ProcessorObject = nullptr;
		InProperty->GetValue(ProcessorObject);
		UClass* ProcessorClass = Cast<UClass>(ProcessorObject);
		if (ProcessorClass)
		{
			UAnimationSharingStateProcessor* Processor = ProcessorClass->GetDefaultObject<UAnimationSharingStateProcessor>();
			return Processor->GetAnimationStateEnum();
		}
	}

	return nullptr;
}

TSharedRef<IPropertyTypeCustomization> FAnimationSetupCustomization::MakeInstance()
{
	return MakeShareable(new FAnimationSetupCustomization());
}

void FAnimationSetupCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	const FName AnimSequencePropertyName = GET_MEMBER_NAME_CHECKED(FAnimationSetup, AnimSequence);
	AnimSequencePropertyHandle = PropertyHandle->GetChildHandle(AnimSequencePropertyName);
	
	HeaderRow
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Center)
		[
			// Show the name of the asset or actor
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.Text_Lambda([this]() -> FText
			{
				if (AnimSequencePropertyHandle.IsValid())
				{
					FText PropertyValueAsName;
					if (AnimSequencePropertyHandle->GetValueAsFormattedText(PropertyValueAsName) == FPropertyAccess::Success)
					{
						return PropertyValueAsName;
					}
				}

				return LOCTEXT("None", "None");
			})			
		]
	];
	/*.ValueContent()
	[
		PropertyHandle->CreatePropertyValueWidget()
	];*/	
}

void FAnimationSetupCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);
	const FName AnimSequencePropertyName = GET_MEMBER_NAME_CHECKED(FAnimationSetup, AnimSequence);
	AnimSequencePropertyHandle = StructPropertyHandle->GetChildHandle(AnimSequencePropertyName);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		IDetailPropertyRow& Property = StructBuilder.AddProperty(ChildHandle);
		
		/** Disable all properties if there is not a valid Animation Sequence provided */
		if (AnimSequencePropertyHandle.IsValid() && (ChildHandle->GetProperty()->GetFName() != AnimSequencePropertyName))
		{
			Property.IsEnabled(TAttribute<bool>::Create([this]() -> bool
			{
				if ( AnimSequencePropertyHandle.Get())
				{
					UObject* ObjectPtr = nullptr;
					if (AnimSequencePropertyHandle->GetValue(ObjectPtr) == FPropertyAccess::Success)
					{
						return ObjectPtr != nullptr;
					}
				}

				return false;
			}));
		}
	}
}

#undef LOCTEXT_NAMESPACE // "AnimationSharingSetupCustomization"

