// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
* The public interface to this module
*/
class FAnimSharingEdModule : public IModuleInterface
{
public:
	FAnimSharingEdModule() {}

	virtual void StartupModule();
	virtual void ShutdownModule();
private:
	class FAssetTypeActions_AnimationSharingSetup* AssetAction;
};
