// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "AnimationSharingSetupFactory.generated.h"

struct FAssetData;
class SWindow;

UCLASS(hidecategories = Object, MinimalAPI)
class UAnimationSharingSetupFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const;
	//~ Begin UFactory Interface
};
