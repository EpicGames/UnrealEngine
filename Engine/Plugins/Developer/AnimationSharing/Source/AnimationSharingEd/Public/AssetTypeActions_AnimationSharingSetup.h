// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions_Base.h"
#include "AnimationSharingSetup.h"

class FAssetTypeActions_AnimationSharingSetup : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimationSharingSetup", "Animation Sharing Setup"); }
	virtual FColor GetTypeColor() const override { return FColor(255, 10, 255); }
	virtual UClass* GetSupportedClass() const override { return UAnimationSharingSetup::StaticClass(); }	
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Animation; }
	virtual const TArray<FText>& GetSubMenus() const override
	{
		static const TArray<FText> SubMenus
		{
			NSLOCTEXT("AssetTypeActions", "AnimAdvancedSubMenu", "Advanced")
		};
		return SubMenus;
	};
};
