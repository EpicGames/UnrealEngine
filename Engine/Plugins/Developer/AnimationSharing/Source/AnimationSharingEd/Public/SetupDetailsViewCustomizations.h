// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

namespace ESelectInfo { enum Type : int; }

static UEnum* GetStateEnumClass(const TSharedPtr<IPropertyHandle>& InProperty);

class FPerSkeletonAnimationSharingSetupCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FPerSkeletonAnimationSharingSetupCustomization() {}
	virtual ~FPerSkeletonAnimationSharingSetupCustomization() = default;

	/** Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** End IPropertyTypeCustomization interface */

protected:
	FText GetSkeletonName() const;

	TSharedPtr<IPropertyHandle> SkeletonPropertyHandle;
};

class FAnimationStateEntryCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FAnimationStateEntryCustomization() : StatePropertyHandle(nullptr), ProcessorPropertyHandle(nullptr), CachedComboBoxEnumClass(nullptr) {}
	virtual ~FAnimationStateEntryCustomization() = default;

	/** Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** End IPropertyTypeCustomization interface */

protected:
	FText GetStateName(TSharedPtr<IPropertyHandle> PropertyHandle) const;
	FDetailWidgetRow& CreateEnumSelectionWidget(TSharedRef<IPropertyHandle> ChildHandle, IDetailChildrenBuilder& StructBuilder);
	const TArray<TSharedPtr<FString>> GetComboBoxSourceItems() const;
	const TSharedPtr<FString> GetSelectedEnum(TSharedPtr<IPropertyHandle> PropertyHandle) const;
	void SelectedEnumChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo, TSharedRef<IPropertyHandle> PropertyHandle);
	void GenerateEnumComboBoxItems();

protected:
	TSharedPtr<IPropertyHandle> StatePropertyHandle;
	TSharedPtr<IPropertyHandle> ProcessorPropertyHandle;

	UEnum* CachedComboBoxEnumClass;
	TArray<TSharedPtr<FString>> ComboBoxItems;
};

class FAnimationSetupCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FAnimationSetupCustomization() {}
	virtual ~FAnimationSetupCustomization() = default;

	/** Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** End IPropertyTypeCustomization interface */

protected:
	FText GetAnimationName(TSharedPtr<IPropertyHandle> PropertyHandle) const;

protected:
	TSharedPtr<IPropertyHandle> AnimSequencePropertyHandle;
};
